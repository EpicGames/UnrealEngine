[/Script/NetworkPrediction.NetworkPredictionSettingsObject]
Settings=(PreferredTickingPolicy=Fixed,ReplicatedManagerClassOverride=None,FixedTickFrameRate=60,bForceEngineFixTickForcePhysics=True,SimulatedProxyNetworkLOD=ForwardPredict,FixedTickInterpolationBufferedMS=100,IndependentTickInterpolationBufferedMS=100,IndependentTickInterpolationMaxBufferedMS=250)
+DevHUDs=(HUDName="Debug",Items=((DisplayName="Debug ON",ExecCommand="np2.debug 1",bAutoBack=False),(DisplayName="Debug OFF",ExecCommand="np2.debug 0",bAutoBack=False),(DisplayName="Debug Draw Tolerance ON",ExecCommand="np2.Debug.Tolerance 1",bAutoBack=False),(DisplayName="Debug Draw Tolerance OFF",ExecCommand="np2.debug.Tolerance 0",bAutoBack=False),(DisplayName="Dump",ExecCommand="np2.dump",bAutoBack=False,bRequirePIE=True),(DisplayName="Spawn Dummy Client",ExecCommand="serverexec Summon /NetworkPredictionExtras/Physics/NetworkPredictionExtras_PhysicsPawnDummy.NetworkPredictionExtras_PhysicsPawnDummy_c",bAutoBack=False,bRequirePIE=false),(DisplayName="Destroy Dummy Clients",ExecCommand="serverexec DestroyAll NetworkPredictionExtras_PhysicsPawnDummy_c",bAutoBack=False,bRequirePIE=false)),bRequirePIE=False,bRequireNotPIE=False)
