// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NetworkPrediction : ModuleRules
	{
		public NetworkPrediction(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"NetCore",
                    "Engine",
                    "RenderCore",
					"PhysicsCore",
					"Chaos",
					"TraceLog"
				}
				);

            // Only needed for the PIE delegate in FNetworkPredictionModule::StartupModule
            if (Target.Type == TargetType.Editor) {
                PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "UnrealEd",
                });
            }

			SetupIrisSupport(Target);
		}
	}
}
