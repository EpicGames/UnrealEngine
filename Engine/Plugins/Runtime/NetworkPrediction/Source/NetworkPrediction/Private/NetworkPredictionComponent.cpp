// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionComponent.h"
#include "Engine/NetConnection.h"
#include "Engine/World.h"
#include "NetworkPredictionWorldManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPredictionComponent)

UNetworkPredictionComponent::UNetworkPredictionComponent()
{
	SetIsReplicatedByDefault(true);
	ServerRPCParamBitWriter.SetAllowResize(true);
}

void UNetworkPredictionComponent::InitializeComponent()
{
	Super::InitializeComponent();

	UWorld* World = GetWorld();	
	UNetworkPredictionWorldManager* NetworkPredictionWorldManager = GetWorld()->GetSubsystem<UNetworkPredictionWorldManager>();
	if (NetworkPredictionWorldManager)
	{
		// Init RepProxies
		ReplicationProxy_ServerRPC.Init(&NetworkPredictionProxy, EReplicationProxyTarget::ServerRPC);
		ReplicationProxy_Autonomous.Init(&NetworkPredictionProxy, EReplicationProxyTarget::AutonomousProxy);
		ReplicationProxy_Simulated.Init(&NetworkPredictionProxy, EReplicationProxyTarget::SimulatedProxy);
		ReplicationProxy_Replay.Init(&NetworkPredictionProxy, EReplicationProxyTarget::Replay);

		InitializeNetworkPredictionProxy();

		CheckOwnerRoleChange();
	}
}

void UNetworkPredictionComponent::EndPlay(const EEndPlayReason::Type Reason)
{
	Super::EndPlay(Reason);
	NetworkPredictionProxy.EndPlay();
}

void UNetworkPredictionComponent::PreReplication(IRepChangedPropertyTracker & ChangedPropertyTracker)
{
	// Only called on the server
	Super::PreReplication(ChangedPropertyTracker);
	
	CheckOwnerRoleChange();

	// We have to update our replication proxies so they can be accurately compared against client shadowstate during property replication. ServerRPC proxy does not need to do this.
	ReplicationProxy_Autonomous.OnPreReplication();
	ReplicationProxy_Simulated.OnPreReplication();
	ReplicationProxy_Replay.OnPreReplication();
}

void UNetworkPredictionComponent::PreNetReceive()
{
	// Only called on the client
	Super::PreNetReceive();
	CheckOwnerRoleChange();
}

void UNetworkPredictionComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME( UNetworkPredictionComponent, NetworkPredictionProxy);
	DOREPLIFETIME_CONDITION( UNetworkPredictionComponent, ReplicationProxy_Autonomous, COND_AutonomousOnly);
	DOREPLIFETIME_CONDITION( UNetworkPredictionComponent, ReplicationProxy_Simulated, COND_SimulatedOnlyNoReplay);
	DOREPLIFETIME_CONDITION( UNetworkPredictionComponent, ReplicationProxy_Replay, COND_ReplayOnly);
}

void UNetworkPredictionComponent::InitializeForNetworkRole(ENetRole Role, const bool bHasNetConnection)
{
	NetworkPredictionProxy.InitForNetworkRole(Role, bHasNetConnection);
}

bool UNetworkPredictionComponent::CheckOwnerRoleChange()
{
	AActor* OwnerActor = GetOwner();
	const ENetRole CurrentRole = OwnerActor->GetLocalRole();
	const bool bHasNetConnection = OwnerActor->GetNetConnection() != nullptr;
	
	if (CurrentRole != NetworkPredictionProxy.GetCachedNetRole() || bHasNetConnection != NetworkPredictionProxy.GetCachedHasNetConnection())
	{
		ReplicationProxy_Autonomous.OwnerActor = OwnerActor;
		ReplicationProxy_Simulated.OwnerActor = OwnerActor;
		ReplicationProxy_Replay.OwnerActor = OwnerActor;

		InitializeForNetworkRole(CurrentRole, bHasNetConnection);
		return true;
	}

	return false;
}

bool UNetworkPredictionComponent::ServerReceiveClientInput_Validate(const FServerReplicationRPCParameter& ProxyParameter)
{
	return true;
}

void UNetworkPredictionComponent::ServerReceiveClientInput_Implementation(const FServerReplicationRPCParameter& ProxyParameter)
{
	// The const_cast is unavoidable here because the replication system only allows by value (forces copy, bad) or by const reference. This use case is unique because we are using the RPC parameter as a temp buffer.
	FServerReplicationRPCParameter& MutableProxyParameter = const_cast<FServerReplicationRPCParameter&>(ProxyParameter);

	const int32 NumIncomingBits = MutableProxyParameter.DataBits.Num();
 	if (NumIncomingBits > FServerReplicationRPCParameter::MaxNumBits)
 	{
 		// Protect against bad data that could cause server to allocate way too much memory.
 		UE_LOGF(LogNetworkPrediction, Error, "ServerReceiveClientInput (%ls): Dropping move due to NumBits (%d) exceeding allowable limit (%d). See FServerReplicationRPCParameter::MaxNumBits.", *GetNameSafe(GetOwner()), NumIncomingBits, FServerReplicationRPCParameter::MaxNumBits);
 		return;
 	}

	// Reuse bit reader to avoid allocating memory each time.
	ServerRPCParamBitReader.SetData((uint8*)MutableProxyParameter.DataBits.GetData(), NumIncomingBits);

	AActor* OwnerActor = GetOwner();
	UNetConnection* NetConnection = OwnerActor->GetNetConnection();

	if (UIrisObjectReferencePackageMap* PackageMap = FServerReplicationRPCParameter::GetIrisPackageMapToReadReferences(NetConnection, MutableProxyParameter.PackageMapExports))
	{
		ServerRPCParamBitReader.PackageMap = PackageMap;	// Iris flow
	}
	else
	{
		ServerRPCParamBitReader.PackageMap = MutableProxyParameter.GetPackageMap();	// Non-Iris / fallback flow
	}

	if (ServerRPCParamBitReader.PackageMap == nullptr)
	{
		UE_LOGF(LogNetworkPrediction, Error, "ServerReceiveClientInput (%ls): Failed to find PackageMap for data serialization!", *GetNameSafe(GetOwner()));
		return;
	}

	// Pass the raw data bits to the proxy parameter for serialized loading
	bool bOutSuccess = true;
	ReplicationProxy_ServerRPC.NetSerialize(ServerRPCParamBitReader, ServerRPCParamBitReader.PackageMap, bOutSuccess);

	if (!bOutSuccess)
	{
		UE_LOGF(LogNetworkPrediction, Error, "ServerReceiveClientInput (%ls): NetSerialize failed!", *GetNameSafe(GetOwner()));
	}
}

void UNetworkPredictionComponent::CallServerRPC()
{
	AActor* OwnerActor = GetOwner();
	
	// Temp hack to make sure the ServerRPC doesn't get suppressed from bandwidth limiting
	// (system hasn't been optimized and not mature enough yet to handle gaps in input stream)
	// Note that this has no effect when using Iris replication.
	FScopedBandwidthLimitBypass BandwidthBypass(OwnerActor);

	// Reset bit writer without affecting allocations
	FBitWriterMark BitWriterReset;
	BitWriterReset.Pop(ServerRPCParamBitWriter);

	// 'static' to avoid reallocation each invocation
	static FServerReplicationRPCParameter ProxyParameter;	
	ProxyParameter.SetReplicationProxy(ReplicationProxy_ServerRPC);

	UNetConnection* NetConnection = OwnerActor->GetNetConnection();

	if (UPackageMap* PackageMap = FServerReplicationRPCParameter::GetIrisPackageMapToCaptureReferences(NetConnection, ProxyParameter.PackageMapExports))
	{
		ServerRPCParamBitWriter.PackageMap = PackageMap;
	}
	else
	{
		// Extract the net package map used for serializing object references.
		ServerRPCParamBitWriter.PackageMap = NetConnection ? ToRawPtr(NetConnection->PackageMap) : nullptr;
	}

	if (ServerRPCParamBitWriter.PackageMap == nullptr)
	{
		UE_LOGF(LogNetworkPrediction, Error, "CallServerRPC: Failed to find a NetConnection/PackageMap for data serialization!");
		return;
	}

	// Reset captured exports stored in ProxyParameter
	ProxyParameter.NetTokensPendingExport.Reset();
	UE::Net::FNetTokenExportScope NetTokenExportScope(ServerRPCParamBitWriter, NetConnection->GetDriver()->GetNetTokenStore(), ProxyParameter.NetTokensPendingExport, "CallServerRPC");

	bool bSerializeSuccess = true;
	if (!ProxyParameter.SerializeToProxy(ServerRPCParamBitWriter, ServerRPCParamBitWriter.PackageMap, bSerializeSuccess))
	{
		return;
	}

	// Copy bits to our proxy param struct, which will be NetSerialized to the server
	ProxyParameter.DataBits.SetNumUninitialized(ServerRPCParamBitWriter.GetNumBits());

	check(ProxyParameter.DataBits.Num() >= ServerRPCParamBitWriter.GetNumBits());
	FMemory::Memcpy(ProxyParameter.DataBits.GetData(), ServerRPCParamBitWriter.GetData(), ServerRPCParamBitWriter.GetNumBytes());

	// Trigger RPC to the server
	ServerReceiveClientInput(ProxyParameter);
}

// --------------------------------------------------------------


