// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionDebug.h"
#include "EngineUtils.h"
#include "DrawDebugHelpers.h"
#include "NetworkPredictionWorldManager.h"
#include "Engine/NetConnection.h"
#include "EngineUtils.h"
#include "UObject/UObjectIterator.h"

namespace NetworkPredictionDebug
{
	void DrawDebugOutline(FTransform Transform, FBox BoundingBox, FColor DrawColor, float LifetimeSeconds)
	{
		static constexpr float Thickness = 2.f;

		npCheck(UNetworkPredictionWorldManager::ActiveInstance != nullptr);
		UWorld* World = UNetworkPredictionWorldManager::ActiveInstance->GetWorld();

		FVector ActorOrigin;
		FVector ActorExtent;
		BoundingBox.GetCenterAndExtents(ActorOrigin, ActorExtent);
		ActorExtent *= Transform.GetScale3D();
		
		DrawDebugBox(World, Transform.GetLocation(), ActorExtent, Transform.GetRotation(), DrawColor, false, LifetimeSeconds, 0, Thickness);
	}

	void DrawDebugText3D(const TCHAR* Str, FTransform Transform, FColor DrawColor, float Lifetime)
	{	
		npCheck(UNetworkPredictionWorldManager::ActiveInstance != nullptr);
		UWorld* World = UNetworkPredictionWorldManager::ActiveInstance->GetWorld();

		DrawDebugString(World, Transform.GetLocation(), Str, nullptr, DrawColor, Lifetime, false);
	}

	FNetworkGUID FindObjectNetGUID(UObject* Obj)
	{
		FNetworkGUID NetGUID;
		if (UWorld* World = Obj->GetWorld())
		{
			if (UNetDriver* NetDriver = World->GetNetDriver())
			{
				if (UNetConnection* NetConnection = NetDriver->ServerConnection)
				{
					NetGUID = NetConnection->PackageMap->GetNetGUIDFromObject(Obj);
				}
				else if (NetDriver->ClientConnections.Num() > 0)
				{
					NetGUID = NetDriver->ClientConnections[0]->PackageMap->GetNetGUIDFromObject(Obj);
				}
			}
		}
		return NetGUID;
	}

	UObject* FindReplicatedObjectOnPIEServer(UObject* ClientObject)
	{
		if (ClientObject == nullptr)
		{
			return nullptr;
		}

		UObject* ServerObject = nullptr;

#if WITH_EDITOR
		FNetworkGUID NetGUID = FindObjectNetGUID(ClientObject);
		if (NetGUID.IsValid())
		{
			// Find the PIE server world
			for (TObjectIterator<UWorld> It; It; ++It)
			{
				if (It->WorldType == EWorldType::PIE && It->GetNetMode() != NM_Client)
				{
					if (UNetDriver* ServerNetDriver = It->GetNetDriver())
					{
						if (ServerNetDriver->ClientConnections.Num() > 0)
						{
							UPackageMap* ServerPackageMap = ServerNetDriver->ClientConnections[0]->PackageMap;
							ServerObject = ServerPackageMap->GetObjectFromNetGUID(NetGUID, true);
							break;
						}
					}
				}
			}
		}
#endif

		return ServerObject;
	}

	
};


// -------------------------------------------------------------------------------------------------------

FAutoConsoleCommandWithWorldAndArgs DrawServerPrimitivesCmd(TEXT("np.Debug.DrawServerPrimitives"), TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray< FString >& Args, UWorld* World) 
{
	static FDelegateHandle BoundHandle;
	static TWeakObjectPtr<UWorld> ServerWorld;

	if (BoundHandle.IsValid())
	{
		FWorldDelegates::OnWorldTickStart.Remove(BoundHandle);
		BoundHandle.Reset();
		return;
	}

	BoundHandle = FWorldDelegates::OnWorldTickStart.AddLambda([&](UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds)
	{
		if (InWorld->GetNetMode() == NM_DedicatedServer)
		{
			ServerWorld = InWorld;
			return;
		}

		if (InWorld->GetNetMode() != NM_Client || !ServerWorld.IsValid())
		{
			return;
		}

		for (TActorIterator<AActor> It(ServerWorld.Get()); It; ++It)
		{
			if (UPrimitiveComponent* PrimitiveComponent = It->FindComponentByClass<UPrimitiveComponent>())
			{
				AActor* Actor = PrimitiveComponent->GetOwner();

				FBox LocalSpaceBox = Actor->CalculateComponentsBoundingBoxInLocalSpace();
				const float Thickness = 2.f;

				FVector ActorOrigin;
				FVector ActorExtent;
				LocalSpaceBox.GetCenterAndExtents(ActorOrigin, ActorExtent);
				ActorExtent *= Actor->GetActorScale3D();
				DrawDebugBox(InWorld, Actor->GetActorLocation(), ActorExtent, Actor->GetActorQuat(), FColor::Purple, false, -1.f, 0, Thickness);
			}
		}
	});
}));
