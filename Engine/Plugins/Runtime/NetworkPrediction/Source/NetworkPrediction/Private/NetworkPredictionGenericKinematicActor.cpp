// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionGenericKinematicActor.h"
#include "NetworkPredictionModelDefRegistry.h"

NP_MODEL_REGISTER(FGenericKinematicActorDef);