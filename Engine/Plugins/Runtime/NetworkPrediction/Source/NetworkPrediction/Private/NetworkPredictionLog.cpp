// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionLog.h"

DEFINE_LOG_CATEGORY(LogNetworkPrediction);