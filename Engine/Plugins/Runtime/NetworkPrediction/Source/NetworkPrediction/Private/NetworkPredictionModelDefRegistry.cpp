// Copyright Epic Games, Inc. All Rights Reserved.
#include "NetworkPredictionModelDefRegistry.h"

FNetworkPredictionModelDefRegistry& FNetworkPredictionModelDefRegistry::Get()
{
	static FNetworkPredictionModelDefRegistry Singleton;

	return Singleton;
}

void FNetworkPredictionModelDefRegistry::FinalizeTypes()
{
	if (bFinalized)
	{
		return;
	}

	bFinalized = true;
	ModelDefList.Sort([](const FTypeInfo& LHS, const FTypeInfo& RHS) -> bool
	{
		if (LHS.SortPriority == RHS.SortPriority)
		{
			UE_LOGF(LogNetworkPrediction, Log, "ModelDefs %ls and %ls have same sort priority. Using lexical sort as backup", LHS.Name, RHS.Name);
			int32 StrCmpResult = FCString::Strcmp(LHS.Name, RHS.Name);
			npEnsureMsgf(StrCmpResult != 0, TEXT("Duplicate ModelDefs appear to have been registered."));
			return StrCmpResult > 0;
		}

		return LHS.SortPriority < RHS.SortPriority; 
	});

	int32 Count = 1;
	for (FTypeInfo& TypeInfo : ModelDefList)
	{
		*TypeInfo.IDPtr = Count++;
	}
}
