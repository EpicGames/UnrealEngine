// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionModule.h"
#include "NetworkPredictionTrace.h"
#include "String/ParseTokens.h"
#include "NetworkPredictionModelDefRegistry.h"
#include "Misc/CoreDelegates.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ISettingsModule.h"
#else
#include "Engine/World.h"
#endif


#define LOCTEXT_NAMESPACE "FNetworkPredictionModule"

class FNetworkPredictionModule : public INetworkPredictionModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange);
	void FinalizeNetworkPredictionTypes();

	FDelegateHandle PieHandle;
	FDelegateHandle ModulesChangedHandle;
	FDelegateHandle WorldPreInitHandle;
};

void FNetworkPredictionModule::StartupModule()
{
	// Disable by default unless in the command line args. This is temp as the existing insights -trace parsing happen before the plugin is loaded
	UE::Trace::ToggleChannel(TEXT("NetworkPredictionChannel"), false);

	FString EnabledChannels;
	FParse::Value(FCommandLine::Get(), TEXT("-trace="), EnabledChannels, false);
	UE::String::ParseTokens(EnabledChannels, TEXT(","), [](FStringView Token) {
		if (Token.Compare(TEXT("NetworkPrediction"), ESearchCase::IgnoreCase)==0 || Token.Compare(TEXT("NP"), ESearchCase::IgnoreCase)==0)
		{
		UE::Trace::ToggleChannel(TEXT("NetworkPredictionChannel"), true);
		}
	});

	ModulesChangedHandle = FModuleManager::Get().OnModulesChanged().AddRaw(this, &FNetworkPredictionModule::OnModulesChanged);

	// Finalize types if the engine is up and running, or register for callback for when it is
	if (GIsRunning)
	{
		FinalizeNetworkPredictionTypes();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
		{
			this->FinalizeNetworkPredictionTypes();
		});
	}
	
	this->WorldPreInitHandle = FWorldDelegates::OnPreWorldInitialization.AddLambda([this](UWorld* World, const UWorld::InitializationValues IVS)
	{
		UE_NP_TRACE_WORLD_PREINIT();
	});

#if WITH_EDITOR
	PieHandle = FEditorDelegates::PreBeginPIE.AddLambda([](const bool bBegan)
	{
		UE_NP_TRACE_PIE_START();
	});

	ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
	if (SettingsModule != nullptr)
	{
		SettingsModule->RegisterSettings("Project", "Project", "Network Prediction",
			LOCTEXT("NetworkPredictionSettingsName", "Network Prediction"),
			LOCTEXT("NetworkPredictionSettingsDescription", "Settings for the Network Prediction runtime module."),
			GetMutableDefault<UNetworkPredictionSettingsObject>()
		);
	}
#endif
}


void FNetworkPredictionModule::ShutdownModule()
{
	if (ModulesChangedHandle.IsValid())
	{
		FModuleManager::Get().OnModulesChanged().Remove(ModulesChangedHandle);
		ModulesChangedHandle.Reset();
	}

	if (WorldPreInitHandle.IsValid())
	{
		FWorldDelegates::OnPreWorldInitialization.Remove(WorldPreInitHandle);
		WorldPreInitHandle.Reset();
	}

#if WITH_EDITOR
	FEditorDelegates::PreBeginPIE.Remove(PieHandle);

	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterSettings("Project", "Project", "Network Prediction");
	}
#endif
}

void FNetworkPredictionModule::OnModulesChanged(FName ModuleThatChanged, EModuleChangeReason ReasonForChange)
{
	// If we haven't finished loading, don't do per module finalizing
	if (GIsRunning == false)
	{
		return;
	}

	switch (ReasonForChange)
	{
	case EModuleChangeReason::ModuleLoaded:
		FinalizeNetworkPredictionTypes();
		break;

	case EModuleChangeReason::ModuleUnloaded:
		FinalizeNetworkPredictionTypes();
		break;
	}
}

void FNetworkPredictionModule::FinalizeNetworkPredictionTypes()
{
	FGlobalCueTypeTable::Get().FinalizeCueTypes();
	FNetworkPredictionModelDefRegistry::Get().FinalizeTypes();
}

IMPLEMENT_MODULE( FNetworkPredictionModule, NetworkPrediction )
#undef LOCTEXT_NAMESPACE

