// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionNetSerializers.h"
#include "NetworkPredictionConfig.h"
#include "NetworkPredictionLog.h"
#include "NetworkPredictionProxy.h"
#include "NetworkPredictionReplicationProxy.h"
#include "Engine/NetConnection.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"
#include "Iris/Serialization/IrisPackageMapExportUtil.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPredictionNetSerializers)



namespace UE::Net
{

struct FNetworkPredictionProxyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Types
	struct FQuantizedData
	{
		int32 SpawnID;
		FNetworkPredictionInstanceArchetype InstanceArchetype;

		bool IsEqual(const FQuantizedData& Other) const
		{
			return (SpawnID == Other.SpawnID && 
			       InstanceArchetype.TickingMode == Other.InstanceArchetype.TickingMode);
		}
	};

	typedef FNetworkPredictionProxy SourceType;
	typedef FQuantizedData QuantizedType;
	typedef FNetworkPredictionProxyNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);

	// Avoid overwriting some members in the target FNetworkPredictionProxy instance.
	static void Apply(FNetSerializationContext&, const FNetApplyArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;

		bool QuantizedTypeMeetRequirements() const;
		void InitNetSerializer();

		inline static const FName NetworkPredictionProxyNetSerializerName = FName("NetworkPredictionProxy");
		UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(NetworkPredictionProxyNetSerializerName, FNetworkPredictionProxyNetSerializer);
	};

	inline static FNetworkPredictionProxyNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};

UE_NET_IMPLEMENT_SERIALIZER(FNetworkPredictionProxyNetSerializer);

// FNetworkPredictionProxyNetSerializer implementation
void FNetworkPredictionProxyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	// Write to the network bitstream from the quantized representation
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	WritePackedInt32(Writer, Value.SpawnID);
	Writer->WriteBits((uint8)Value.InstanceArchetype.TickingMode, 8u);
}

void FNetworkPredictionProxyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	// Read a quantized representation from the network bitstream
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	Target.SpawnID = ReadPackedInt32(Reader);

	const uint8 TickingModeAsByte = static_cast<uint8>(Reader->ReadBits(8u));
	Target.InstanceArchetype.TickingMode = static_cast<ENetworkPredictionTickingPolicy>(TickingModeAsByte);
}


void FNetworkPredictionProxyNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	// Create a quantized representation from the full instance shadow copy
	SourceType& Source = *reinterpret_cast<SourceType*>(Args.Source);
	QuantizedType& Target = *reinterpret_cast<QuantizedType*>(Args.Target);

	Target.SpawnID = Source.GetID();
	Target.InstanceArchetype.TickingMode = Source.GetCachedArchetype().TickingMode;
}

void FNetworkPredictionProxyNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	// Write onto a full shadow copy instance from a quantized representation
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	Target.GetID() = FNetworkPredictionID(Source.SpawnID, Target.GetID().GetTraceID());
	Target.GetCachedArchetype().TickingMode = Source.InstanceArchetype.TickingMode;
}

bool FNetworkPredictionProxyNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& QuantizedValue0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& QuantizedValue1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);
		return QuantizedValue0.IsEqual(QuantizedValue1);
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);

		return Value0.Identical(&Value1, 0);
	}
}


void FNetworkPredictionProxyNetSerializer::Apply(FNetSerializationContext& Context, const FNetApplyArgs& Args)
{
	// Copy selectively from the shadow copy to the real instance that game-level systems see, then let it configure itself
	SourceType& Source = *reinterpret_cast<SourceType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	const int32 NewID = Source.GetID();	// ID will be set on Target inside ConfigureAfterLoading
	Target.GetCachedArchetype().TickingMode = Source.GetCachedArchetype().TickingMode;

	Target.ConfigureAfterLoading(NewID);
}

// FNetSerializerRegistryDelegates
FNetworkPredictionProxyNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(NetworkPredictionProxyNetSerializerName);
}

void FNetworkPredictionProxyNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(NetworkPredictionProxyNetSerializerName);
}


} // end namespace UE::Net




// FServerReplicationRPCParameterNetSerializer implementation
//  Largely a copy of FCharacterNetworkSerializationPackedBitsNetSerializer to start
namespace UE::Net::Private
{
	static constexpr inline uint32 CalculateRequiredWordCount(uint32 NumBits) { return (NumBits + NumBitsPerDWORD - 1U) / NumBitsPerDWORD; }
}


namespace UE::Net
{

	struct FServerReplicationRPCParameterNetSerializerQuantizedType
	{
		typedef uint32 WordType;
		static constexpr uint32 DefaultReserveNumBits = FServerReplicationRPCParameter::NumReservedBits;
		static constexpr uint32 InlinedWordCount = Private::CalculateRequiredWordCount(DefaultReserveNumBits);

		typedef FNetSerializerArrayStorage<WordType, AllocationPolicies::TInlinedElementAllocationPolicy<InlinedWordCount>> FDataBitsStorage;

		FIrisPackageMapExportsQuantizedType QuantizedExports;
		FDataBitsStorage DataBitsStorage;
		uint32 NumDataBits = 0;
	};

}

template <> struct TIsPODType<UE::Net::FServerReplicationRPCParameterNetSerializerQuantizedType> { enum { Value = true }; };

namespace UE::Net
{

struct FServerReplicationRPCParameterNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bHasCustomNetReference = true;

	typedef uint32 WordType;

	typedef FServerReplicationRPCParameter SourceType;
	typedef FServerReplicationRPCParameterNetSerializerQuantizedType QuantizedType;
	typedef struct FServerReplicationRPCParameterNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs&);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
	};

	static void FreeDynamicStateInternal(FNetSerializationContext&, QuantizedType& Value);

	static FServerReplicationRPCParameterNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};

UE_NET_IMPLEMENT_SERIALIZER(FServerReplicationRPCParameterNetSerializer);


const FServerReplicationRPCParameterNetSerializer::ConfigType FServerReplicationRPCParameterNetSerializer::DefaultConfig;
FServerReplicationRPCParameterNetSerializer::FNetSerializerRegistryDelegates FServerReplicationRPCParameterNetSerializer::NetSerializerRegistryDelegates;

void FServerReplicationRPCParameterNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	// For now we ignore this in default state hash due to complications with asymmetrically serialized state.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);

	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// Serialize captured references and exports
	FIrisPackageMapExportsUtil::Serialize(Context, Value.QuantizedExports);

	// Write data bits
	const uint32 NumDataBits = Value.NumDataBits;
	if (Writer->WriteBool(NumDataBits > 0))
	{
		UE::Net::WritePackedUint32(Writer, NumDataBits);
		Writer->WriteBitStream(Value.DataBitsStorage.GetData(), 0, NumDataBits);
	}
}

// static
void FServerReplicationRPCParameterNetSerializer::FreeDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Value)
{
	// Free quantized state for captured reference and exports
	FIrisPackageMapExportsUtil::FreeDynamicState(Context, Value.QuantizedExports);

	Value.DataBitsStorage.Free(Context);
	Value.NumDataBits = 0;
}

void FServerReplicationRPCParameterNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	// For consistency, we should never get here. For now we ignore this in default state hash due to complications with asymmetrically serialized state.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Deserialize captured references and exports
	FIrisPackageMapExportsUtil::Deserialize(Context, TargetValue.QuantizedExports);

	const bool bHasDataBits = Reader->ReadBool();
	if (bHasDataBits)
	{
		const uint32 NumDataBits = UE::Net::ReadPackedUint32(Reader);

		const uint32 MaxNumDataBits = Config->MaxQuantizedSizeBits;
		if (NumDataBits > MaxNumDataBits)
		{
			Context.SetError(GNetError_ArraySizeTooLarge);
			UE_LOGF(LogNetworkPrediction, Error, "FServerReplicationRPCParameterNetSerializer::Deserialize: Invalidating RPC param due to NumBits (%u) exceeding allowable limit (%u).", NumDataBits, MaxNumDataBits);
			ensureMsgf(false, TEXT("Invalidating received FServerReplicationRPCParameterRPC due to NumBits exceeding allowable limit"));
			return;
		}

		const uint32 RequiredWordCount = Private::CalculateRequiredWordCount(NumDataBits);
		TargetValue.DataBitsStorage.AdjustSize(Context, RequiredWordCount);

		Reader->ReadBitStream(TargetValue.DataBitsStorage.GetData(), NumDataBits);
		TargetValue.NumDataBits = NumDataBits;
	}
	else
	{
		TargetValue.DataBitsStorage.Free(Context);
		TargetValue.NumDataBits = 0U;
	}
}

void FServerReplicationRPCParameterNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	// Create a quantized representation from a full shadow copy instance
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	// Quantize captured references and exports
	FIrisPackageMapExportsUtil::Quantize(Context, SourceValue.PackageMapExports, MakeArrayView(SourceValue.NetTokensPendingExport), TargetValue.QuantizedExports);

	uint32 NumDataBits = SourceValue.DataBits.Num();

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const uint32 MaxNumDataBits = Config->MaxQuantizedSizeBits;

	if (NumDataBits > MaxNumDataBits)
	{
		// This is just to avoid disconnect and instead warn and invalidate the data on the sending side.
		UE_LOGF(LogNetworkPrediction, Error, "FServerReplicationRPCParameterNetSerializer::Quantize: Invalidating RPC param due to NumBits (%u) exceeding allowable limit (%u). See FServerReplicationRPCParameterNetSerializerConfig::DefaultMaxQuantizedSizeBits.", NumDataBits, MaxNumDataBits);
		NumDataBits = 0U;
		ensureMsgf(false, TEXT("Invalidating RPC param due to NumBits exceeding allowable limit"));
	}

	TargetValue.DataBitsStorage.AdjustSize(Context, Private::CalculateRequiredWordCount(NumDataBits));
	if (NumDataBits > 0)
	{
		FMemory::Memcpy(TargetValue.DataBitsStorage.GetData(), SourceValue.DataBits.GetData(), (NumDataBits + 7U) / 8U);
	}
	TargetValue.NumDataBits = NumDataBits;
}

void FServerReplicationRPCParameterNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	// Write onto a full shadow copy instance from a quantized representation
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	// Dequantize captured references and exports and inject into target
	FIrisPackageMapExportsUtil::Dequantize(Context, Source.QuantizedExports, Target.PackageMapExports);

	// DataBits
	Target.DataBits.SetNumUninitialized(Source.NumDataBits);
	Target.DataBits.SetRangeFromRange(0, Source.NumDataBits, Source.DataBitsStorage.GetData());
}

bool FServerReplicationRPCParameterNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.NumDataBits != Value1.NumDataBits)
		{
			return false;
		}

		// Compare references and exports
		if (!FIrisPackageMapExportsUtil::IsEqual(Context, Value0.QuantizedExports, Value1.QuantizedExports))
		{
			return false;
		}

		const uint32 RequiredWords = Private::CalculateRequiredWordCount(Value0.NumDataBits);
		if (RequiredWords > 0 && FMemory::Memcmp(Value0.DataBitsStorage.GetData(), Value1.DataBitsStorage.GetData(), sizeof(WordType) * RequiredWords) != 0)
		{
			return false;
		}
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);
		return Value0.DataBits == Value1.DataBits;
	}

	return true;
}

bool FServerReplicationRPCParameterNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);

	const uint32 MaxNumDataBits = Config->MaxQuantizedSizeBits;

	if (SourceValue.NumDataBits > MaxNumDataBits)
	{
		return false;
	}

	if (!FIrisPackageMapExportsUtil::Validate(Context, SourceValue.QuantizedExports))
	{
		return false;
	}

	return true;
}

void FServerReplicationRPCParameterNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	FIrisPackageMapExportsUtil::CloneDynamicState(Context, TargetValue.QuantizedExports, SourceValue.QuantizedExports);

	TargetValue.DataBitsStorage.Clone(Context, SourceValue.DataBitsStorage);
	TargetValue.NumDataBits = SourceValue.NumDataBits;
}

void FServerReplicationRPCParameterNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	FreeDynamicStateInternal(Context, Value);
}

void FServerReplicationRPCParameterNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetReferenceCollector& Collector = *reinterpret_cast<UE::Net::FNetReferenceCollector*>(Args.Collector);

	FIrisPackageMapExportsUtil::CollectNetReferences(Context, Value.QuantizedExports, Args.ChangeMaskInfo, Collector);
}

static const FName PropertyNetSerializerRegistry_NAME_ServerReplicationRPCParameter("ServerReplicationRPCParameter");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ServerReplicationRPCParameter, FServerReplicationRPCParameterNetSerializer);


FServerReplicationRPCParameterNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ServerReplicationRPCParameter);
}

void FServerReplicationRPCParameterNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ServerReplicationRPCParameter);
}

} // end UE::Net


namespace UE::Net
{

	struct FFReplicationProxyNetSerializerQuantizedType
	{
		typedef uint32 WordType;
		static constexpr uint32 DefaultReserveNumBits = FReplicationProxy::NumReservedBits;
		static constexpr uint32 InlinedWordCount = Private::CalculateRequiredWordCount(DefaultReserveNumBits);

		typedef FNetSerializerArrayStorage<WordType, AllocationPolicies::TInlinedElementAllocationPolicy<InlinedWordCount>> FDataBitsStorage;

		FIrisPackageMapExportsQuantizedType QuantizedExports;
		// How many bytes the current allocation can hold.
		uint32 ByteCapacity = 0;
		FDataBitsStorage DataBitsStorage;
		uint32 NumDataBits = 0;

		int32 CachedPendingFrame = INDEX_NONE;
	};

}

template <> struct TIsPODType<UE::Net::FFReplicationProxyNetSerializerQuantizedType> { enum { Value = true }; };


namespace UE::Net
{

struct FReplicationProxyNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bHasDynamicState = true;
	static constexpr bool bHasCustomNetReference = true;

	// Types

	typedef FReplicationProxy SourceType;
	typedef FFReplicationProxyNetSerializerQuantizedType QuantizedType;
	typedef FReplicationProxyNetSerializerConfig ConfigType;

	inline static const ConfigType DefaultConfig;

	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	
	static void Apply(FNetSerializationContext&, const FNetApplyArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	static constexpr uint32 AllocationAlignment = 4U;

	static void FreeDynamicStateInternal(FNetSerializationContext&, QuantizedType& Value);
	static void GrowDynamicStateInternal(FNetSerializationContext&, QuantizedType& Value, uint32 NewBitCount);
	static void AdjustStorageSize(FNetSerializationContext&, QuantizedType& Value, uint32 NewBitCount);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
	};

	static FReplicationProxyNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};


UE_NET_IMPLEMENT_SERIALIZER(FReplicationProxyNetSerializer);


FReplicationProxyNetSerializer::FNetSerializerRegistryDelegates FReplicationProxyNetSerializer::NetSerializerRegistryDelegates;


void FReplicationProxyNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();

	// If we have any captured exports, serialize them.
	FIrisPackageMapExportsUtil::Serialize(Context, Value.QuantizedExports);

	// Write data.
	const uint32 NumDataBits = Value.NumDataBits;
	if (Writer->WriteBool(NumDataBits > 0))
	{
		UE::Net::WritePackedUint32(Writer, NumDataBits);
		Writer->WriteBitStream(Value.DataBitsStorage.GetData(), 0, NumDataBits);
	}
}

void FReplicationProxyNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	// For consistency, we should never get here. For now we ignore this in default state hash due to complications with asymmetrically serialized state.
	if (Context.IsInitializingDefaultState())
	{
		return;
	}

	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);
	const uint32 CurrentBitCount = Value.NumDataBits;

	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	// Read exports for packagemap.
	FIrisPackageMapExportsUtil::Deserialize(Context, Value.QuantizedExports);

	const bool bHasDataBits = Reader->ReadBool();

	if (bHasDataBits)
	{
		// Read the data
		const uint32 NewBitCount = ReadPackedUint32(Reader);

		if (!ensureMsgf(NewBitCount <= Config->MaxQuantizedSizeBits,
			TEXT("FReplicationProxyNetSerializer::Deserialize data size of %u bits exceeds maximum of %u bits."),
			NewBitCount, Config->MaxQuantizedSizeBits))
		{
			Context.SetError(GNetError_ArraySizeTooLarge);
			return;
		}

		AdjustStorageSize(Context, Value, NewBitCount);

		Reader->ReadBitStream(static_cast<uint32*>(Value.DataBitsStorage.GetData()), NewBitCount);
	}
	else
	{
		Value.DataBitsStorage.Free(Context);
		Value.NumDataBits = 0U;
	}
}


void FReplicationProxyNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const ConfigType* Config = static_cast<const ConfigType*>(Args.NetSerializerConfig);
	SourceType& Source = *reinterpret_cast<SourceType*>(Args.Source);
	QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Target);

	Value.CachedPendingFrame = Source.GetCachedPendingFrame();

	UIrisObjectReferencePackageMap* PackageMap = nullptr; 

	if (const AActor* ReplicatingActor = Source.OwnerActor.Get())
	{
		if (UEngineReplicationBridge* ReplicationBridge = UE::Net::FReplicationSystemUtil::GetActorReplicationBridge(ReplicatingActor))
		{
			PackageMap = ReplicationBridge->GetObjectReferencePackageMap();
		}
	}

	// Since this struct uses custom serialization path we need to explicitly capture exports in order to forward them to iris
	UE::Net::FIrisPackageMapExports PackageMapExports;
	UE::Net::FNetTokenExportContext::FNetTokenExports NetTokensPendingExport;

	if (PackageMap)
	{
		PackageMap->InitForWrite(&PackageMapExports);
	}

	// Use the Property serialization and store as binary blob.
	FNetBitWriter Archive(PackageMap, FReplicationProxy::NumReservedBits);
	FNetTokenExportScope NetTokenExportScope(Archive, Context.GetNetTokenStore(), NetTokensPendingExport);

	if (!Context.IsInitializingDefaultState())
	{
		bool bOutSuccess = true;
		Source.NetSerialize(Archive, PackageMap, bOutSuccess);
	}

	const uint64 BitCount = Archive.GetNumBits();

	if (!ensureMsgf(BitCount <= Config->MaxQuantizedSizeBits,
		TEXT("FReplicationProxyNetSerializer::Quantize: data size of %llu bits exceeds maximum of %u bits."),
		BitCount, Config->MaxQuantizedSizeBits))
	{
		Context.SetError(GNetError_ArraySizeTooLarge);
		return;
	}

	if (!ensureMsgf(!Archive.IsError(), TEXT("FReplicationProxyNetSerializer::Quantize: NetBitWriter archive error in NetSerialize")))
	{
		Context.SetError(GNetError_InternalError);
		return;
	}

	// Quantize captured exports
	FIrisPackageMapExportsUtil::Quantize(Context, PackageMapExports, NetTokensPendingExport, Value.QuantizedExports);

	// Deal with serialized data
	AdjustStorageSize(Context, Value, static_cast<uint32>(BitCount));
	if (BitCount > 0)
	{
		FMemory::Memcpy(Value.DataBitsStorage.GetData(), Archive.GetData(), (BitCount + 7U) / 8U);
	}
}

void FReplicationProxyNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	// Dequantize captured references and exports and inject into target
	FIrisPackageMapExportsUtil::Dequantize(Context, Source.QuantizedExports, Target.PackageMapExports);

	// DataBits
	Target.DataBits.SetNumUninitialized(Source.NumDataBits);
	Target.DataBits.SetRangeFromRange(0, Source.NumDataBits, Source.DataBitsStorage.GetData());
}

bool FReplicationProxyNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		return (Value0.CachedPendingFrame == Value1.CachedPendingFrame);
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);
		return Value0.Identical(&Value1, 0);
	}
}

void FReplicationProxyNetSerializer::Apply(FNetSerializationContext& Context, const FNetApplyArgs& Args)
{
	const SourceType& Source = *reinterpret_cast<const SourceType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	UIrisObjectReferencePackageMap* PackageMap = nullptr;

	if (const AActor* ReplicatingActor = Target.OwnerActor.Get())
	{
		if (UEngineReplicationBridge* ReplicationBridge = UE::Net::FReplicationSystemUtil::GetActorReplicationBridge(ReplicatingActor))
		{
			PackageMap = ReplicationBridge->GetObjectReferencePackageMap();
		}
	}

	// Setup resolve context for call into NetSerialize
	UE::Net::FNetTokenResolveContext ResolveContext;
	ResolveContext.RemoteNetTokenStoreState = Context.GetRemoteNetTokenStoreState();
	ResolveContext.NetTokenStore = Context.GetNetTokenStore();

	if (PackageMap)
	{
		PackageMap->InitForRead(&Source.PackageMapExports, ResolveContext);
	}

	uint32 NumDataBits = Source.DataBits.Num();

	if (NumDataBits > 0)
	{
		// Note that we are reading from Source's data bits and net serializing directly from them, 
		// without copying the bits to Target. This avoids an unnecessary memcpy.
		FNetBitReader Archive(PackageMap, static_cast<uint8*>((void*)Source.DataBits.GetData()), NumDataBits);
		bool bOutSuccess = true;
		Target.NetSerialize(Archive, PackageMap, bOutSuccess);
	}
}


void FReplicationProxyNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	const QuantizedType& SourceValue = *reinterpret_cast<const QuantizedType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	FIrisPackageMapExportsUtil::CloneDynamicState(Context, TargetValue.QuantizedExports, SourceValue.QuantizedExports);
	TargetValue.DataBitsStorage.Clone(Context, SourceValue.DataBitsStorage);
	TargetValue.NumDataBits = SourceValue.NumDataBits;
	TargetValue.ByteCapacity = SourceValue.ByteCapacity;
	TargetValue.CachedPendingFrame = SourceValue.CachedPendingFrame;
}

void FReplicationProxyNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FreeDynamicStateInternal(Context, *reinterpret_cast<QuantizedType*>(Args.Source));
}

void FReplicationProxyNetSerializer::FreeDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Value)
{
	// Free quantized state for captured reference and exports
	FIrisPackageMapExportsUtil::FreeDynamicState(Context, Value.QuantizedExports);

	Value.DataBitsStorage.Free(Context);
	Value.NumDataBits = 0;
}

void FReplicationProxyNetSerializer::GrowDynamicStateInternal(FNetSerializationContext& Context, QuantizedType& Value, uint32 NewBitCount)
{
	checkSlow(NewBitCount > Value.NumDataBits);

	const uint32 ByteCount = Align((NewBitCount + 7U) / 8U, AllocationAlignment);

	const uint32 RequiredWordCount = Private::CalculateRequiredWordCount(NewBitCount);
	Value.DataBitsStorage.AdjustSize(Context, RequiredWordCount);

	// Clear the last word to support IsEqual Memcmp optimization.
	const uint32 LastWordIndex = ByteCount / 4U - 1U;
	static_cast<uint32*>(Value.DataBitsStorage.GetData())[LastWordIndex] = 0U;

	Value.ByteCapacity = ByteCount;
	Value.NumDataBits = NewBitCount;
}


void FReplicationProxyNetSerializer::AdjustStorageSize(FNetSerializationContext& Context, QuantizedType& Value, uint32 NewBitCount)
{
	const uint32 NewByteCapacity = Align((NewBitCount + 7U) / 8U, AllocationAlignment);
	if (NewByteCapacity == 0)
	{
		// Free everything
		FreeDynamicStateInternal(Context, Value);
	}
	else if (NewByteCapacity > Value.ByteCapacity)
	{
		GrowDynamicStateInternal(Context, Value, NewBitCount);
	}
	// If byte capacity is within the allocated capacity we just update the bit count and clear the last word
	else
	{
		Value.NumDataBits = NewBitCount;

		// Clear the last word to support IsEqual Memcmp optimization.
		const uint32 LastWordIndex = NewByteCapacity / 4U - 1U;
		static_cast<uint32*>(Value.DataBitsStorage.GetData())[LastWordIndex] = 0U;
	}
}

void FReplicationProxyNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	const ConfigType& Config = *static_cast<const ConfigType*>(Args.NetSerializerConfig);
	const QuantizedType& Value = *reinterpret_cast<const QuantizedType*>(Args.Source);
	FNetReferenceCollector& Collector = *reinterpret_cast<UE::Net::FNetReferenceCollector*>(Args.Collector);

	FIrisPackageMapExportsUtil::CollectNetReferences(Context, Value.QuantizedExports, Args.ChangeMaskInfo, Collector);
}

static const FName PropertyNetSerializerRegistry_NAME_ReplicationProxy("ReplicationProxy");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ReplicationProxy, FReplicationProxyNetSerializer);


FReplicationProxyNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ReplicationProxy);
}

void FReplicationProxyNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_ReplicationProxy);
}



} // end of UE::Net





