// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/Serialization/NetSerializer.h"
#include "UObject/ObjectMacros.h"
#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionNetSerializers.generated.h"


// Iris net serialization for FNetworkPredictionProxy. 
// This is required due to the struct mixing locally-set and replicated member variables.
USTRUCT()
struct FNetworkPredictionProxyNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()
};

namespace UE::Net
{
	UE_NET_DECLARE_SERIALIZER(FNetworkPredictionProxyNetSerializer, NETWORKPREDICTION_API);
}


// Iris net serialization for FServerReplicationRPCParameter. 
// This is required due to the struct using a custom NetSerialize function that is routed to other serialization method(s) that may do polymorphic and/or conditional serialization.
// Implementation is based on FCharacterNetworkSerializationPackedBitsNetSerializer
USTRUCT()
struct FServerReplicationRPCParameterNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	static constexpr uint32 DefaultMaxQuantizedSizeBits = FServerReplicationRPCParameter::MaxNumBits;

	// Value used to sanity check incoming data so that we do not over-allocate dynamic memory
	UPROPERTY()
	uint32 MaxQuantizedSizeBits = DefaultMaxQuantizedSizeBits;
};

namespace UE::Net
{

UE_NET_DECLARE_SERIALIZER(FServerReplicationRPCParameterNetSerializer, NETWORKPREDICTION_API);

}


// Iris net serialization for FReplicationProxy. 
// This is required due to the struct using a custom NetSerialize function that is routed to other serialization method(s) that may do polymorphic and/or conditional serialization.
// Implementation is partly based on the last resort net serializer.
USTRUCT()
struct FReplicationProxyNetSerializerConfig : public FNetSerializerConfig
{
	GENERATED_BODY()

public:
	static constexpr uint32 DefaultMaxQuantizedSizeBits = FReplicationProxy::MaxNumBits;

	// Value used to sanity check incoming data so that we do not over-allocate dynamic memory
	UPROPERTY()
	uint32 MaxQuantizedSizeBits = DefaultMaxQuantizedSizeBits;
};

namespace UE::Net
{
	UE_NET_DECLARE_SERIALIZER(FReplicationProxyNetSerializer, NETWORKPREDICTION_API);
}