// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionReplicatedManager.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Engine/World.h"
#include "NetworkPredictionWorldManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPredictionReplicatedManager)


ANetworkPredictionReplicatedManager::FOnAuthoritySpawn ANetworkPredictionReplicatedManager::OnAuthoritySpawnDelegate;
TWeakObjectPtr<ANetworkPredictionReplicatedManager> ANetworkPredictionReplicatedManager::AuthorityInstance;

ANetworkPredictionReplicatedManager::ANetworkPredictionReplicatedManager()
{
	bReplicates = true;
	NetPriority = 1000.f; // We want this to be super high priority when it replicates
	// Mute very low update frequency ensure. Was 0.001f .
	SetNetUpdateFrequency(0.125f); // Low frequency: we will use ForceNetUpdate when important data changes
	bAlwaysRelevant = true;
}

void ANetworkPredictionReplicatedManager::BeginPlay()
{
	Super::BeginPlay();
	if (GetLocalRole() == ROLE_Authority)
	{
		OnAuthoritySpawnDelegate.Broadcast(this);
	}
	else
	{
		UNetworkPredictionWorldManager* NetworkPredictionWorldManager = GetWorld()->GetSubsystem<UNetworkPredictionWorldManager>();
		npCheckSlow(NetworkPredictionWorldManager);
		NetworkPredictionWorldManager->ReplicatedManager = this;
	}
}

void ANetworkPredictionReplicatedManager::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	FDoRepLifetimeParams SharedParams;
	SharedParams.bIsPushBased = true;

	DOREPLIFETIME_WITH_PARAMS_FAST(ANetworkPredictionReplicatedManager, SharedPackageMap, SharedParams);
}

FDelegateHandle ANetworkPredictionReplicatedManager::OnAuthoritySpawn(const TFunction<void(ANetworkPredictionReplicatedManager*)>& Func)
{
	if (AuthorityInstance.IsValid())
	{
		Func(AuthorityInstance.Get());
	}

	// I don't think there is a way to move a TUniqueFunction onto a delegate, so TFunction will have to do
	return OnAuthoritySpawnDelegate.AddLambda(Func);
}

uint8 ANetworkPredictionReplicatedManager::GetIDForObject(UObject* Obj) const
{
	// Naive lookup
	for (auto It = SharedPackageMap.Items.CreateConstIterator(); It; ++It)
	{
		const FSharedPackageMapItem& Item = *It;
		if (Item.SoftPtr.Get() == Obj)
		{
			npCheckSlow(It.GetIndex() < TNumericLimits<uint8>::Max());
			return (uint8)It.GetIndex();
		}
	}

	npEnsureMsgf(false, TEXT("Could not find Object %s in SharedPackageMap."), *GetNameSafe(Obj));
	return 0;
}

TSoftObjectPtr<UObject> ANetworkPredictionReplicatedManager::GetObjectForID(uint8 ID) const
{
	if (SharedPackageMap.Items.IsValidIndex(ID))
	{
		return SharedPackageMap.Items[ID].SoftPtr;
	}

	return TSoftObjectPtr<UObject>();
}

uint8 ANetworkPredictionReplicatedManager::AddObjectToSharedPackageMap(TSoftObjectPtr<UObject> SoftPtr)
{
	if (SharedPackageMap.Items.Num()+1 >= TNumericLimits<uint8>::Max())
	{
		UE_LOGF(LogTemp, Warning, "Mock SharedPackageMap has overflowed!");
		for (FSharedPackageMapItem& Item : SharedPackageMap.Items)
		{
			UE_LOGF(LogTemp, Warning, "   %ls", *Item.SoftPtr.ToString());
		}
		ensureMsgf(false, TEXT("SharedPackageMap overflowed"));
		return 0;
	}

	SharedPackageMap.Items.Add(FSharedPackageMapItem{SoftPtr});
	MARK_PROPERTY_DIRTY_FROM_NAME(ANetworkPredictionReplicatedManager, SharedPackageMap, this);

	return SharedPackageMap.Items.Num()-1;
}
