// Copyright Epic Games, Inc. All Rights Reserved

#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionLog.h"
#include "NetworkPredictionProxy.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"
#include "Net/Iris/ReplicationSystem/EngineReplicationBridge.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Engine/NetConnection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPredictionReplicationProxy)

// -------------------------------------------------------------------------------------------------------------------------------
//	FReplicationProxy
// -------------------------------------------------------------------------------------------------------------------------------

void FReplicationProxy::Init(FNetworkPredictionProxy* InNetSimProxy, EReplicationProxyTarget InReplicationTarget)
{
	NetSimProxy = InNetSimProxy;
	ReplicationTarget = InReplicationTarget;
}

bool FReplicationProxy::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	if (NetSerializeFunc)
	{
		FNetSerializeParams P(Ar, Map);
		NetSerializeFunc(P);
		bOutSuccess = !Ar.IsError();
		return true;
	}
	else
	{
		UE_LOGF(LogNetworkPrediction, Verbose, "NetSerializeFunc not set for FReplicationProxy %d. This may occur as replication begins if using Iris, but should subside once the object's net role is established.", EnumToUnderlyingType(ReplicationTarget));
	}

	bOutSuccess = false;
	return true;
}

void FReplicationProxy::OnPreReplication()
{
	if (NetSimProxy)
	{
		CachedPendingFrame = NetSimProxy->GetPendingFrame();
	}
}

bool FReplicationProxy::Identical(const FReplicationProxy* Other, uint32 PortFlags) const
{
	return (CachedPendingFrame == Other->CachedPendingFrame);
}

// -------------------------------------------------------------------------------------------------------------------------------
//	FServerReplicationRPCParameter
// -------------------------------------------------------------------------------------------------------------------------------
bool FServerReplicationRPCParameter::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	CachedPackageMap = Map;

	// Array size in bits, using minimal number of bytes to write it out.
	uint32 NumBits = DataBits.Num();
	Ar.SerializeIntPacked(NumBits);

	if (NumBits > FServerReplicationRPCParameter::MaxNumBits)
	{
		// Protect against bad data that could cause server to allocate way too much memory.
		UE_LOGF(LogNetworkPrediction, Error, "FServerReplicationRPCParameter::NetSerialize: Dropping incoming RPC param due to NumBits (%d) exceeding allowable limit (%d).", NumBits, FServerReplicationRPCParameter::MaxNumBits);
		bOutSuccess = false;
		return true;
	}

	if (Ar.IsLoading())
	{
		DataBits.Init(0, NumBits);
	}
	else if (Ar.IsSaving() && NetTokensPendingExport.Num())
	{
		// As we now support exporting NetTokens from shared serialization and FServerReplicationRPCParameter serializes data outside of the normal flow
		// we explicitly capture exports which we needs to be inject during actual serialization.
		if (UE::Net::FNetTokenExportContext* ExportContext = UE::Net::FNetTokenExportContext::GetNetTokenExportContext(Ar))
		{
			ExportContext->AppendNetTokensPendingExport(NetTokensPendingExport);
		}
	}

	// Array data
	Ar.SerializeBits(DataBits.GetData(), NumBits);

	bOutSuccess = !Ar.IsError();
	return true; 


}

bool FServerReplicationRPCParameter::SerializeToProxy(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	check (Ar.IsSaving()); // This function is only for writing to the archive from the client's proxy
	check (Proxy);

	Proxy->NetSerialize(Ar, Map, bOutSuccess);

	return !Ar.IsError();
}

//static 
UIrisObjectReferencePackageMap* FServerReplicationRPCParameter::GetIrisPackageMapToCaptureReferences(UNetConnection* NetConnection, UE::Net::FIrisPackageMapExports& TargetPkgMapExports)
{
	using namespace UE::Net;

	if (const UEngineReplicationBridge* Bridge = FReplicationSystemUtil::GetActorReplicationBridge(NetConnection))
	{
		if (UIrisObjectReferencePackageMap* ObjectReferencePackageMap = Bridge->GetObjectReferencePackageMap())
		{
			ObjectReferencePackageMap->InitForWrite(&TargetPkgMapExports);

			return ObjectReferencePackageMap;
		}
	}

	return nullptr;
}

//static
UIrisObjectReferencePackageMap* FServerReplicationRPCParameter::GetIrisPackageMapToReadReferences(const UNetConnection* NetConnection, UE::Net::FIrisPackageMapExports& TargetPkgMapExports)
{
	using namespace UE::Net;

	if (const UEngineReplicationBridge* Bridge = FReplicationSystemUtil::GetActorReplicationBridge(NetConnection))
	{
		if (UIrisObjectReferencePackageMap* ObjectReferencePackageMap = Bridge->GetObjectReferencePackageMap())
		{
			const UReplicationSystem* ReplicationSystem = UE::Net::FReplicationSystemUtil::GetReplicationSystem(NetConnection->GetDriver());
			ObjectReferencePackageMap->InitForRead(&TargetPkgMapExports, ReplicationSystem->GetNetTokenResolveContext(NetConnection->GetConnectionHandle().GetParentConnectionId()));

			return ObjectReferencePackageMap;
		}
	}

	return nullptr;
}

// -------------------------------------------------------------------------------------------------------------------------------
//	FScopedBandwidthLimitBypass
// -------------------------------------------------------------------------------------------------------------------------------

FScopedBandwidthLimitBypass::FScopedBandwidthLimitBypass(AActor* OwnerActor)
{
	if (OwnerActor)
	{
		CachedNetConnection = OwnerActor->GetNetConnection();
		if (CachedNetConnection)
		{
			RestoreBits = CachedNetConnection->QueuedBits + CachedNetConnection->SendBuffer.GetNumBits();
		}
	}
}

FScopedBandwidthLimitBypass::~FScopedBandwidthLimitBypass()
{
	if (CachedNetConnection)
	{
		CachedNetConnection->QueuedBits = RestoreBits - CachedNetConnection->SendBuffer.GetNumBits();
	}
}

