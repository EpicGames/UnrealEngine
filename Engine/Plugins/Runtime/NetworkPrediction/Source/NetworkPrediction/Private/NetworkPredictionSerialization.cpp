// Copyright Epic Games, Inc. All Rights Reserved.


#include "NetworkPredictionSerialization.h"


namespace NetworkPredictionCVars
{
	TAutoConsoleVariable<bool> EnableTimeDilation(
		TEXT("np.TimeDilation.Enable"),
		true,
		TEXT("Time Dilation affects Autonomous Proxy clients in Fixed Tick mode, trying to minimize the amount of time the client is predicting ahead of the server. The server suggests to slow down or speed up the frequency of the client's sim frames, to encourage the smallest amount of input buffering without starvation."));

	TAutoConsoleVariable<float> TimeDilationAmount(
		TEXT("np.TimeDilation.Amount"),
		0.01f,
		TEXT("Affects how quickly dilation occurs. Server-side CVar, Disable TimeDilation by setting to 0 | Default: 0.01 | Value is in percent where 0.01 = 1% dilation. Example: 1.0/0.01 = 100, meaning that over the time it usually takes to tick 100 fixed simulation steps we will tick 99 or 101 depending on if we dilate up or down."));

	TAutoConsoleVariable<bool> AllowTimeDilationEscalation(
		TEXT("np.TimeDilation.AllowEscalation"),
		true,
		TEXT("Enables aggressive dilation when input buffer surplus or deficit grow. Server-side CVar. If enabled, dilate the time even more depending on how many ticks we need to adjust. When set to false, we use the set TimeDilationAmount as perform correct the offset slowly. When set to true, we multiply the TimeDilationEscalationDecay with the buffer offset count and resolve faster. Capped by TimeDilationEscalationDecayMax."));

	TAutoConsoleVariable<float> TimeDilationEscalationDecay(
		TEXT("np.TimeDilation.EscalationDecay"),
		0.05f,
		TEXT("Value is a multiplier, Default: 0.05. For each frame offset from the ideal amount, also decay by this much. Disable by setting to 0"));

	TAutoConsoleVariable<float> TimeDilationEscalationDecayMax(
		TEXT("np.TimeDilation.EscalationDecayMax"),
		0.5f,
		TEXT("Default: 0.5. The max decay value for escalated time dilation, which caps decay when input buffer offset is large. Lower value means higher decay."));

	TAutoConsoleVariable<float> TimeDilationMax(
		TEXT("np.TimeDilation.Max"),
		1.1f,
		TEXT("Max value of the time dilation multiplier, with 1.0 meaning no dilation"));

	TAutoConsoleVariable<float> TimeDilationMin(
		TEXT("np.TimeDilation.Min"),
		0.9f,
		TEXT("Min value of the time dilation multiplier, with 1.0 meaning no dilation"));


}