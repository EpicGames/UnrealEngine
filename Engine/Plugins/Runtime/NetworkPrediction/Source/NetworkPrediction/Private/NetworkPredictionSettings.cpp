// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionSettings.h"
#include "Engine/Engine.h"
#include "NetworkPredictionWorldManager.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/HUD.h"
#include "Engine/Canvas.h"
#include "Engine/LocalPlayer.h"
#include "Engine/LevelScriptActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPredictionSettings)

#if WITH_EDITOR
void UNetworkPredictionSettingsObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	for (TObjectIterator<UNetworkPredictionWorldManager> It; It; ++It)
	{
		if (!It->HasAnyFlags(RF_ClassDefaultObject))
		{
			It->SyncNetworkPredictionSettings(this);
		}
	}
};
#endif

namespace UE_NETWORK_PREDICTION
{
	struct FDevMenuState
	{
		bool bIsActive = false;
		int32 ActiveHUD = INDEX_NONE;

		float LastMaxX = 0.f;
		float LastMaxY = 0.f;
		
		TArray<FNetworkPredictionDevHUD> DevHUDs;
	};

	FORCENOINLINE void DrawText(FDevMenuState& State, UCanvas* Canvas, const TCHAR* Str, float& X, float& Y, bool bEnabled=true)
	{
		float XL = 0.f;
		float YL = 0.f;

		if (bEnabled)
		{
			Canvas->SetDrawColor(FColor::White);
		}
		else
		{
			Canvas->SetDrawColor(FColor(150, 150, 150));
		}
		
		Canvas->StrLen(GEngine->GetMediumFont(), Str, XL, YL);		
		Y += Canvas->DrawText(GEngine->GetMediumFont(), Str, X, Y);
		
		State.LastMaxY = Y;
		State.LastMaxX = FMath::Max<float>(State.LastMaxX, X + XL);
	}

	FORCENOINLINE void DrawDevHUD(TMap<TWeakObjectPtr<UWorld>, FDevMenuState>& WorldHandlesMap, AHUD* HUD, UCanvas* Canvas)
	{
		FDevMenuState& State = WorldHandlesMap.FindOrAdd(HUD->GetWorld());
		if (!State.bIsActive)
		{
			return;
		}

		float YPos = Canvas->SizeY * 0.45f; //455.f; //16;
		float XPos = 24.f;
		float Pad = 4.f;

		FCanvasTileItem TileItem(FVector2D(XPos-Pad, YPos-Pad), FVector2D(State.LastMaxX - XPos + (Pad*2.f), State.LastMaxY - YPos + (Pad*2.f)), FLinearColor(0.05f, 0.05f, 0.05f, 0.25f));
		Canvas->DrawItem(TileItem);

		State.LastMaxX = 0.f;
		State.LastMaxY = 0.f;

		//DrawText(State, Canvas, TEXT("Dev HUD "), XPos, YPos);
		//DrawText(State, Canvas, TEXT(" "), XPos, YPos);

		if (State.ActiveHUD == INDEX_NONE)
		{
			for (int32 idx=0; idx < State.DevHUDs.Num(); ++idx)
			{
				const FNetworkPredictionDevHUD& DevHUD = State.DevHUDs[idx];
				const bool bEnabled = (!DevHUD.bRequireNotPIE || !GIsEditor) && (!DevHUD.bRequirePIE || GIsEditor);

				DrawText(State, Canvas, *FString::Printf(TEXT("[%d] %s"), idx+1, *DevHUD.HUDName), XPos, YPos, bEnabled);
			}

			DrawText(State, Canvas, TEXT(" "), XPos, YPos);
			DrawText(State, Canvas, TEXT("[0] HIDE Dev HUD"), XPos, YPos);
		}
		else if (State.DevHUDs.IsValidIndex(State.ActiveHUD))
		{
			const FNetworkPredictionDevHUD& DevHUD = State.DevHUDs[State.ActiveHUD];
			for (int32 idx=0; idx < DevHUD.Items.Num(); ++idx)
			{
				const FNetworkPredictionDevHUDItem& HUDItem = DevHUD.Items[idx];
				const bool bEnabled = (!HUDItem.bRequireNotPIE || !GIsEditor) && (!HUDItem.bRequirePIE || GIsEditor);
				DrawText(State, Canvas, *FString::Printf(TEXT("[%d] %s"), idx+1, *HUDItem.DisplayName), XPos, YPos, bEnabled);
			}

			DrawText(State, Canvas, TEXT(" "), XPos, YPos);
			DrawText(State, Canvas, TEXT("[0] Back"), XPos, YPos);
		}
		else
		{
			DrawText(State, Canvas, TEXT("???"), XPos, YPos);
		}		
	}

	FORCENOINLINE void DevMenu(const TArray< FString >& Args, UWorld* InWorld)
	{
		static TMap<TWeakObjectPtr<UWorld>, FDevMenuState> WorldHandlesMap;	

		FDevMenuState& State = WorldHandlesMap.FindOrAdd(InWorld);

		int32 Choice = INDEX_NONE;
		if (Args.Num() > 0)
		{
			LexFromString(Choice, *Args[0]);
		}

		if (Choice == 0)
		{
			if (State.ActiveHUD == INDEX_NONE)
			{
				State.bIsActive = !State.bIsActive;
			}
			else
			{
				State.ActiveHUD = INDEX_NONE;
			}
			return;
		}

		if (State.DevHUDs.Num() <= 0)
		{
			const UNetworkPredictionSettingsObject* Settings = GetDefault<UNetworkPredictionSettingsObject>();
			check(Settings);
			State.DevHUDs = Settings->DevHUDs;

			ALevelScriptActor* LevelScript = InWorld->GetLevelScriptActor();
			FNetworkPredictionDevHUD& LevelScriptDevHUD = State.DevHUDs.InsertDefaulted_GetRef(0);
			LevelScriptDevHUD.HUDName = TEXT("Level Script");
			if (LevelScript)
			{
				for (TFieldIterator<UFunction> FunctionIt(LevelScript->GetClass(), EFieldIteratorFlags::ExcludeSuper); FunctionIt; ++FunctionIt)
				{
					UFunction* Func = *FunctionIt;
					if (Func->HasAnyFunctionFlags(EFunctionFlags::FUNC_UbergraphFunction | EFunctionFlags::FUNC_Event))
					{
						continue;
					}
					if (!Func->HasAllFunctionFlags(EFunctionFlags::FUNC_BlueprintCallable))
					{
						continue;
					}
					
					FString CommandStr = FString::Printf(TEXT("ServerExec \"ce \\\"%s\\\""), *Func->GetName());
					for (TFieldIterator<FProperty> It(Func); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
					{
						CommandStr += TEXT(" 0");
					}
					CommandStr += TEXT("\"");

					FNetworkPredictionDevHUDItem& Item = LevelScriptDevHUD.Items.AddDefaulted_GetRef();
					//Item.DisplayName = FString::Printf(TEXT("%s 0x%X"), *FunctionIt->GetName(), (uint32)Func->FunctionFlags);
					Item.DisplayName = FunctionIt->GetName();
					Item.ExecCommand = CommandStr;
					Item.bAutoBack = false;

					if (LevelScriptDevHUD.Items.Num() == 9)
					{
						break;
					}
				}
			}
		}

		if (!State.bIsActive)
		{
			State.bIsActive = true;
		}
		else if (State.ActiveHUD == INDEX_NONE)
		{
			if (State.DevHUDs.IsValidIndex(Choice-1))
			{
				if ((!State.DevHUDs[Choice-1].bRequirePIE || GIsEditor) && (!State.DevHUDs[Choice-1].bRequireNotPIE || !GIsEditor))
				{
					State.ActiveHUD = Choice-1;
				}
			}
		}
		else
		{
			if (State.DevHUDs.IsValidIndex(State.ActiveHUD))
			{
				const FNetworkPredictionDevHUD& DevHUD = State.DevHUDs[State.ActiveHUD];
				if (DevHUD.Items.IsValidIndex(Choice-1))
				{
					const FNetworkPredictionDevHUDItem& Item = DevHUD.Items[Choice-1];
					if ((!Item.bRequirePIE || GIsEditor) && (!Item.bRequireNotPIE || !GIsEditor))
					{
						if (ULocalPlayer* LocalPlayer = InWorld->GetFirstLocalPlayerFromController())
						{
							LocalPlayer->Exec(InWorld, *Item.ExecCommand, *GLog);
							if (Item.bAutoBack)
							{
								State.ActiveHUD = INDEX_NONE;
							}
						}
					}
				}
			}
		}

		// One time init:
		TWeakObjectPtr<UWorld> WeakWorld(InWorld);
		bool bRegisteredInput  = false;
		auto BindInput = [&bRegisteredInput, WeakWorld]()
		{
			if (UWorld* World = WeakWorld.Get())
			{
				if (ULocalPlayer* LocalPlayer = World->GetFirstLocalPlayerFromController())
				{
					//LocalPlayer->Exec(InWorld, *FString::Printf(TEXT("setbind %d \"np2.DevMenu %d\n"), i, i), *GLog);

					LocalPlayer->Exec(World, TEXT("setbind one \"np2.DevMenu 1\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind two \"np2.DevMenu 2\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind three \"np2.DevMenu 3\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind four \"np2.DevMenu 4\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind five \"np2.DevMenu 5\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind six \"np2.DevMenu 6\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind seven \"np2.DevMenu 7\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind eight \"np2.DevMenu 8\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind nine \"np2.DevMenu 9\""), *GLog);
					LocalPlayer->Exec(World, TEXT("setbind zero \"np2.DevMenu 0\""), *GLog);
					bRegisteredInput = true;
				}
			}
		};

		if (!bRegisteredInput)
		{
			BindInput();
		}

		static FDelegateHandle Handle;
		if (Handle.IsValid())
		{
			return;
		}

		Handle = AHUD::OnHUDPostRender.AddLambda([](AHUD* HUD, UCanvas* Canvas)
		{
			DrawDevHUD(WorldHandlesMap, HUD, Canvas);
		});
	}
}

FAutoConsoleCommandWithWorldAndArgs DevMenuToggleCmd(TEXT("np2.DevMenu"), TEXT(""),
	FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray< FString >& Args, UWorld* InWorld) 
{
	UE_NETWORK_PREDICTION::DevMenu(Args, InWorld);
	
}));
