// Copyright Epic Games, Inc. All Rights Reserved.


#include "NetworkPredictionTrace.h"
#include "Containers/StringFwd.h"
#include "Engine/GameInstance.h"
#include "Engine/GameInstance.h"
#include "Engine/World.h"
#include "NetworkPredictionLog.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

// TODO:
// Should update string tracing with UE::Trace::AnsiString

namespace NetworkPredictionTraceInternal
{
	enum class ENetworkPredictionTraceVersion : uint32
	{
		Initial = 1,
	};

	static constexpr ENetworkPredictionTraceVersion NetworkPredictionTraceVersion = ENetworkPredictionTraceVersion::Initial;
};

UE_TRACE_CHANNEL_DEFINE(NetworkPredictionChannel, "Traces network prediction system operations including simulation state, client/server reconciliation, \
rollback injection, input commands, sync state, auxiliary state, physics state, buffered input, fixed tick offsets, \
out-of-band state modifications, and system faults for debugging networked gameplay prediction.")

UE_TRACE_EVENT_BEGIN(NetworkPrediction, SimScope)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
UE_TRACE_EVENT_END()

// Trace a simulation creation. GroupName is attached as attachment.
UE_TRACE_EVENT_BEGIN(NetworkPrediction, SimulationCreated)
	UE_TRACE_EVENT_FIELD(uint32, SimulationID) // server assigned (shared client<->server)
	UE_TRACE_EVENT_FIELD(int32, TraceID) // process unique id
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, DebugName)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, SimulationConfig)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
	UE_TRACE_EVENT_FIELD(uint8, NetRole)
	UE_TRACE_EVENT_FIELD(uint8, bHasNetConnection)
	UE_TRACE_EVENT_FIELD(uint8, TickingPolicy)
	UE_TRACE_EVENT_FIELD(uint8, NetworkLOD)
	UE_TRACE_EVENT_FIELD(int32, ServiceMask)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, SimulationScope)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, SimState)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, Version)
	UE_TRACE_EVENT_FIELD(uint32, Version)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, WorldPreInit)
	UE_TRACE_EVENT_FIELD(uint64, EngineFrameNumber)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, PieBegin)
	UE_TRACE_EVENT_FIELD(uint64, EngineFrameNumber)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, WorldFrameStart)
	UE_TRACE_EVENT_FIELD(uint64, EngineFrameNumber)
	UE_TRACE_EVENT_FIELD(float, DeltaSeconds)
UE_TRACE_EVENT_END()

// General system fault. Log message is in attachment
UE_TRACE_EVENT_BEGIN(NetworkPrediction, SystemFault)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Message)
UE_TRACE_EVENT_END()

// Traces general tick state (called before ticking N sims)
UE_TRACE_EVENT_BEGIN(NetworkPrediction, Tick)
	UE_TRACE_EVENT_FIELD(int32, StartMS)
	UE_TRACE_EVENT_FIELD(int32, DeltaMS)
	UE_TRACE_EVENT_FIELD(int32, OutputFrame)
UE_TRACE_EVENT_END()

// Signals that the given sim has done a tick. Expected to be called after the 'Tick' event has been traced
UE_TRACE_EVENT_BEGIN(NetworkPrediction, SimTick)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
UE_TRACE_EVENT_END()

// Signals that we are in are receiving a NetSerialize function
UE_TRACE_EVENT_BEGIN(NetworkPrediction, NetRecv)
	UE_TRACE_EVENT_FIELD(int32, Frame)
	UE_TRACE_EVENT_FIELD(int32, TimeMS)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, ShouldReconcile)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, Reconcile)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, UserString)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, RollbackInject)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, PushInputFrame)
	UE_TRACE_EVENT_FIELD(int32, Frame)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, FixedTickOffset)
	UE_TRACE_EVENT_FIELD(int32, Offset)
	UE_TRACE_EVENT_FIELD(bool, Changed)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, BufferedInput)
	UE_TRACE_EVENT_FIELD(int32, NumBufferedFrames)
	UE_TRACE_EVENT_FIELD(bool, bFault)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, ProduceInput)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, OOBStateMod)
	UE_TRACE_EVENT_FIELD(int32, TraceID)
	UE_TRACE_EVENT_FIELD(int32, Frame)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Source)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, InputCmd)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Value)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, SyncState)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Value)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(NetworkPrediction, AuxState)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Value)
UE_TRACE_EVENT_END()

// ---------------------------------------------------------------------------

void FNetworkPredictionTrace::TraceSimulationCreated_Internal(FNetworkPredictionID ID, FStringBuilderBase& Builder)
{
	const uint16 AttachmentSize = Builder.Len() * sizeof(FStringBuilderBase::ElementType);

	UE_TRACE_LOG(NetworkPrediction, SimulationCreated, NetworkPredictionChannel)
		<< SimulationCreated.SimulationID((int32)ID)
		<< SimulationCreated.TraceID(ID.GetTraceID())
		<< SimulationCreated.DebugName(Builder.ToString(), Builder.Len());
}

void FNetworkPredictionTrace::TraceWorldFrameStart(UGameInstance* GameInstance, float DeltaSeconds)
{
	if (!GameInstance || GameInstance->GetWorld()->GetNetMode() == NM_Standalone)
	{
		// No networking yet, don't start tracing
		return;
	}

	UE_TRACE_LOG(NetworkPrediction, WorldFrameStart, NetworkPredictionChannel)
		<< WorldFrameStart.EngineFrameNumber(GFrameNumber)
		<< WorldFrameStart.DeltaSeconds(DeltaSeconds);
}

void FNetworkPredictionTrace::TraceSimulationConfig(int32 TraceID, ENetRole NetRole, bool bHasNetConnection, const FNetworkPredictionInstanceArchetype& Archetype, const FNetworkPredictionInstanceConfig& Config, int32 ServiceMask)
{
	npEnsureMsgf(NetRole != ENetRole::ROLE_None && NetRole != ENetRole::ROLE_MAX, TEXT("Invalid NetRole %d"), NetRole);

	UE_TRACE_LOG(NetworkPrediction, SimulationConfig, NetworkPredictionChannel)
		<< SimulationConfig.TraceID(TraceID)
		<< SimulationConfig.NetRole((uint8)NetRole)
		<< SimulationConfig.bHasNetConnection((uint8)bHasNetConnection)
		<< SimulationConfig.TickingPolicy((uint8)Archetype.TickingMode)
		<< SimulationConfig.ServiceMask(ServiceMask);		
}

void FNetworkPredictionTrace::TraceSimulationScope(int32 TraceID)
{
	UE_TRACE_LOG(NetworkPrediction, SimulationScope, NetworkPredictionChannel)
		<< SimulationScope.TraceID(TraceID);
}

void FNetworkPredictionTrace::TraceSimState(int32 TraceID)
{
	UE_TRACE_LOG(NetworkPrediction, SimState, NetworkPredictionChannel)
		<< SimState.TraceID(TraceID);
}

void FNetworkPredictionTrace::TraceTick(int32 StartMS, int32 DeltaMS, int32 OutputFrame)
{
	UE_TRACE_LOG(NetworkPrediction, Tick, NetworkPredictionChannel)
		<< Tick.StartMS(StartMS)
		<< Tick.DeltaMS(DeltaMS)
		<< Tick.OutputFrame(OutputFrame);
}

void FNetworkPredictionTrace::TraceSimTick(int32 TraceID)
{
	UE_TRACE_LOG(NetworkPrediction, SimTick, NetworkPredictionChannel)
		<< SimTick.TraceID(TraceID);
}

void FNetworkPredictionTrace::TraceUserState_Internal(ETraceUserState StateType, FAnsiStringBuilderBase& Builder)
{
	switch(StateType)
	{
		case ETraceUserState::Input:
		{
			UE_TRACE_LOG(NetworkPrediction, InputCmd, NetworkPredictionChannel)
				<< InputCmd.Value(Builder.GetData(), Builder.Len());
			break;
		}
		case ETraceUserState::Sync:
		{
			UE_TRACE_LOG(NetworkPrediction, SyncState, NetworkPredictionChannel)
				<< SyncState.Value(Builder.GetData(), Builder.Len());
			break;
		}
		case ETraceUserState::Aux:
		{
			UE_TRACE_LOG(NetworkPrediction, AuxState, NetworkPredictionChannel)
				<< AuxState.Value(Builder.GetData(), Builder.Len());
			break;
		}
	}
}

void FNetworkPredictionTrace::TraceNetRecv(int32 Frame, int32 TimeMS)
{
	UE_TRACE_LOG(NetworkPrediction, NetRecv, NetworkPredictionChannel)
		<< NetRecv.Frame(Frame)
		<< NetRecv.TimeMS(TimeMS);
}

void FNetworkPredictionTrace::TraceReconcile(const FAnsiStringView& StrView)
{
	UE_TRACE_LOG(NetworkPrediction, Reconcile, NetworkPredictionChannel)
		<< Reconcile.UserString(StrView.GetData(), StrView.Len());
}

void FNetworkPredictionTrace::TraceShouldReconcile(int32 TraceID)
{
	UE_TRACE_LOG(NetworkPrediction, ShouldReconcile, NetworkPredictionChannel)
		<< ShouldReconcile.TraceID(TraceID);
}

void FNetworkPredictionTrace::TraceRollbackInject(int32 TraceID)
{
	UE_TRACE_LOG(NetworkPrediction, RollbackInject, NetworkPredictionChannel)
		<< RollbackInject.TraceID(TraceID);
}

void FNetworkPredictionTrace::TracePIEStart()
{
	UE_TRACE_LOG(NetworkPrediction, PieBegin, NetworkPredictionChannel)
		<< PieBegin.EngineFrameNumber(GFrameNumber);
}

void FNetworkPredictionTrace::TraceWorldPreInit()
{
	UE_TRACE_LOG(NetworkPrediction, Version, NetworkPredictionChannel)
		<< Version.Version((uint32)NetworkPredictionTraceInternal::NetworkPredictionTraceVersion);

	UE_TRACE_LOG(NetworkPrediction, WorldPreInit, NetworkPredictionChannel)
		<< WorldPreInit.EngineFrameNumber(GFrameNumber);
}

void FNetworkPredictionTrace::TracePushInputFrame(int32 Frame)
{
	UE_TRACE_LOG(NetworkPrediction, PushInputFrame, NetworkPredictionChannel)
		<< PushInputFrame.Frame(Frame);
}

void FNetworkPredictionTrace::TraceFixedTickOffset(int32 Offset, bool bChanged)
{
	UE_TRACE_LOG(NetworkPrediction, FixedTickOffset, NetworkPredictionChannel)
		<< FixedTickOffset.Offset(Offset)
		<< FixedTickOffset.Changed(bChanged);
}

void FNetworkPredictionTrace::TraceBufferedInput(int32 NumBufferedFrames, bool bFault)
{
	UE_TRACE_LOG(NetworkPrediction, BufferedInput, NetworkPredictionChannel)
		<< BufferedInput.NumBufferedFrames(NumBufferedFrames)
		<< BufferedInput.bFault(bFault);
}

void FNetworkPredictionTrace::TraceProduceInput(int32 TraceID)
{
	UE_TRACE_LOG(NetworkPrediction, ProduceInput, NetworkPredictionChannel)
		<< ProduceInput.TraceID(TraceID);
}

void FNetworkPredictionTrace::TraceOOBStateMod(int32 TraceID, int32 Frame, const FAnsiStringView& StrView)
{
	UE_TRACE_LOG(NetworkPrediction, OOBStateMod, NetworkPredictionChannel)
		<< OOBStateMod.TraceID(TraceID)
		<< OOBStateMod.Frame(Frame)
		<< OOBStateMod.Source(StrView.GetData(), StrView.Len());
}

void FNetworkPredictionTrace::TraceSystemFault(const TCHAR* Fmt, ...)
{
	TStringBuilder<512> Builder;

	va_list Args;
	va_start(Args, Fmt);
	Builder.AppendV(Fmt, Args);
	va_end(Args);

	UE_LOGF(LogNetworkPrediction, Log, "SystemFault: %ls", Builder.ToString());

	UE_TRACE_LOG(NetworkPrediction, SystemFault, NetworkPredictionChannel)
		<< SystemFault.Message(Builder.GetData(), Builder.Len());
}
