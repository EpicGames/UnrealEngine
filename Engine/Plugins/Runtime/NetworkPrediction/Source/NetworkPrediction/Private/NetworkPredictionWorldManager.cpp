// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkPredictionWorldManager.h"
#include "Engine/Engine.h"
#include "PBDRigidsSolver.h"
#include "ChaosSolversModule.h"
#include "NetworkPredictionReplicatedManager.h"
#include "Services/NetworkPredictionService_Finalize.inl"
#include "Services/NetworkPredictionService_Input.inl"
#include "Services/NetworkPredictionService_Interpolate.inl"
#include "Services/NetworkPredictionService_Rollback.inl"
#include "Services/NetworkPredictionService_ServerRPC.inl"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NetworkPredictionWorldManager)

UNetworkPredictionWorldManager* UNetworkPredictionWorldManager::ActiveInstance=nullptr;

// -----------------------------------------------------------------------------------------------
//
// -----------------------------------------------------------------------------------------------

UNetworkPredictionWorldManager::UNetworkPredictionWorldManager()
{

}

void UNetworkPredictionWorldManager::Initialize(FSubsystemCollectionBase& Collection)
{
	UWorld* World = GetWorld();
	check(World);

	if (World->WorldType == EWorldType::PIE || World->WorldType == EWorldType::Game)
	{
		PreTickDispatchHandle = FWorldDelegates::OnWorldTickStart.AddUObject(this, &UNetworkPredictionWorldManager::OnWorldPreTick);
		PostTickDispatchHandle = World->OnPostTickDispatch().AddUObject(this, &UNetworkPredictionWorldManager::ReconcileSimulationsPostNetworkUpdate);
		PreWorldActorTickHandle = FWorldDelegates::OnWorldPreActorTick.AddUObject(this, &UNetworkPredictionWorldManager::BeginNewSimulationFrame);

		SyncNetworkPredictionSettings(GetDefault<UNetworkPredictionSettingsObject>());
	}
}

void UNetworkPredictionWorldManager::Deinitialize()
{
	if (UWorld* World = GetWorld())
	{
		if (PreTickDispatchHandle.IsValid())
		{
			FWorldDelegates::OnWorldTickStart.Remove(PreTickDispatchHandle);
		}
		if (PostTickDispatchHandle.IsValid())
		{
			World->OnPostTickDispatch().Remove(PostTickDispatchHandle);
		}
		if (PreWorldActorTickHandle.IsValid())
		{
			FWorldDelegates::OnWorldPreActorTick.Remove(PreWorldActorTickHandle);
		}
	}
}

void UNetworkPredictionWorldManager::SyncNetworkPredictionSettings(const UNetworkPredictionSettingsObject* SettingsObj)
{
	this->Settings = SettingsObj->Settings;
}

// -----------------------------------------------------------------------------------------------
//
// -----------------------------------------------------------------------------------------------

void UNetworkPredictionWorldManager::OnWorldPreTick(UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds)
{
	if (InWorld != GetWorld())
	{
		return;
	}

	UE_NP_TRACE_WORLD_FRAME_START(InWorld->GetGameInstance(), InDeltaSeconds);

	// Defer to the engine ticking rate, if fixed. Otherwise use NPP's setting.
	const float FixedTickFrameRate = (GEngine && GEngine->bUseFixedFrameRate) ? GEngine->FixedFrameRate : Settings.FixedTickFrameRate;

	OnWorldPreTick_Internal(InDeltaSeconds, FixedTickFrameRate);

	// Instantiate replicated manager on server
	if (!ReplicatedManager && InWorld->GetNetMode() != NM_Client)
	{
		UClass* ReplicatedManagerClass = GetDefault<UNetworkPredictionSettingsObject>()->Settings.ReplicatedManagerClassOverride.Get();
		ReplicatedManager = ReplicatedManagerClass ? InWorld->SpawnActor<ANetworkPredictionReplicatedManager>(ReplicatedManagerClass) : InWorld->SpawnActor<ANetworkPredictionReplicatedManager>();
	}
}

void UNetworkPredictionWorldManager::OnWorldPreTick_Internal(float InDeltaSeconds, float InFixedFrameRate)
{
	// Update fixed tick rate, this can be changed via editor settings
	FixedTickState.FixedStepRealTimeMS = (1.f /  InFixedFrameRate) * 1000.f;
	FixedTickState.FixedStepMS = (int32)FixedTickState.FixedStepRealTimeMS;

	// Time Dilation (Fixed Tick only) : happens only on autonomous clients as a throttle up/down of the client's simulation frequency in order to minimize server-side input buffering.
	// Higher latency network conditions require more input buffering, and the buffer should grow/shrink as network latency changes.
	// This is calculated by the server based on its input buffer count and sent to autonomous proxy via TFixedTickReplicator_AP NetSend().
	FixedTickState.FixedStepDilatedRealTimeMS = FixedTickState.FixedStepRealTimeMS;
	
	if (NetworkPredictionCVars::EnableTimeDilation->GetBool())
	{
		FixedTickState.FixedStepDilatedRealTimeMS = FixedTickState.FixedStepRealTimeMS * FixedTickState.SuggestedTimeDilation;
	}

	ActiveInstance = this;
}

void UNetworkPredictionWorldManager::ReconcileSimulationsPostNetworkUpdate()
{
	UWorld* World = GetWorld();
	if (World->GetNetMode() != NM_Client)
	{
		return;
	}

	ReconcileSimulationsPostNetworkUpdate_Internal();
}

void UNetworkPredictionWorldManager::ReconcileSimulationsPostNetworkUpdate_Internal()
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NP_RECONCILE);
	TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::Reconcile);

	ActiveInstance = this;
	bLockServices = true;

	// Trace Local->Server offset. We need to trace this so that we can flag reconciles that happened
	// due to this (usually caused by server being starved for input)
	const bool OffsetChanged = (LastFixedTickOffset != FixedTickState.Offset);
	UE_NP_TRACE_FIXED_TICK_OFFSET(FixedTickState.Offset, OffsetChanged);
	LastFixedTickOffset = FixedTickState.Offset;

	// -------------------------------------------------------------------------
	//	Non-rollback reconcile services
	// -------------------------------------------------------------------------

	// Don't reconcile FixedTick interpolates until we've started interpolation
	// This makes the service's implementation easier if it can rely on a known
	// ToFrame while reconciling network updates
	if (FixedTickState.Interpolation.ToFrame != INDEX_NONE)
	{
		for (TUniquePtr<IFixedInterpolateService>& Ptr : Services.FixedInterpolate.Array)
		{
			Ptr->Reconcile(&FixedTickState);
		}
	}

	for (TUniquePtr<IIndependentInterpolateService>& Ptr : Services.IndependentInterpolate.Array)
	{
		Ptr->Reconcile(&VariableTickState);
	}

	// -------------------------------------------------------------------------
	//	Fixed Tick rollback
	// -------------------------------------------------------------------------

	// Does anyone need to rollback?
	int32 RollbackFrame = INDEX_NONE;
	for (TUniquePtr<IFixedRollbackService>& Ptr : Services.FixedRollback.Array)
	{
		const int32 ReqFrame = Ptr->QueryRollback(&FixedTickState);
		if (ReqFrame != INDEX_NONE)
		{
			RollbackFrame = (RollbackFrame == INDEX_NONE ? ReqFrame : FMath::Min(RollbackFrame, ReqFrame));
		}
	}

	if (RollbackFrame != INDEX_NONE)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NP_ROLLBACK);
		TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::Rollback);

		if (RollbackFrame < FixedTickState.PendingFrame)
		{
			// Common case: rollback to previously ticked frame and resimulate

			const int32 EndFrame = FixedTickState.PendingFrame;
			const int32 NumFrames = EndFrame - RollbackFrame;
			npEnsureSlow(NumFrames > 0);

			bool bFirstStep = true;

			// Do rollback as necessary
			for (int32 Frame=RollbackFrame; Frame < EndFrame; ++Frame)
			{
				FixedTickState.PendingFrame = Frame;
				FNetSimTimeStep Step = FixedTickState.GetNextTimeStep();
				FServiceTimeStep ServiceStep = FixedTickState.GetNextServiceTimeStep();

				const int32 ServerInputFrame = Frame + FixedTickState.Offset;
				UE_NP_TRACE_PUSH_TICK(Step.TotalSimulationTime, FixedTickState.FixedStepMS, Step.Frame);

				// Everyone must apply corrections and flush as necessary before anyone runs the next sim tick
				// bFirstStep will indicate that even if they don't have a correction, they need to rollback their historic state
				for (TUniquePtr<IFixedRollbackService>& Ptr : Services.FixedRollback.Array)
				{
					Ptr->PreStepRollback(Step, ServiceStep, FixedTickState.Offset, bFirstStep);
				}

				// Run Sim ticks
				for (TUniquePtr<IFixedRollbackService>& Ptr : Services.FixedRollback.Array)
				{
					Ptr->StepRollback(Step, ServiceStep);
				}

				bFirstStep = false;
			}

			FixedTickState.PendingFrame = EndFrame;

		}
		else if (RollbackFrame == FixedTickState.PendingFrame)
		{
			// Correction is at the PendingFrame (frame we haven't ticked yet)
			// For now, just do nothing. We are either in a really bad state of PL or are just starting up
			// As our input frames make the round trip, we'll get some slack and be doing corrections in the above code block
			// (Setting the correction data now most likely is still wrong and not worth the iteration time)

			UE_LOGF(LogNetworkPrediction, Warning, "RollbackFrame %d EQUAL PendingFrame %d... Offset: %d", RollbackFrame, FixedTickState.PendingFrame, FixedTickState.Offset);
		}
		else if (RollbackFrame > FixedTickState.PendingFrame)
		{
			// Most likely we haven't had a confirmed frame yet so our local frame -> server mapping hasn't been set yet
			UE_LOGF(LogNetworkPrediction, Log, "RollbackFrame %d AHEAD of PendingFrame %d... Offset: %d", RollbackFrame, FixedTickState.PendingFrame, FixedTickState.Offset);
		}
	}

	// -------------------------------------------------------------------------
	//	Independent Tick rollback
	// -------------------------------------------------------------------------
	for (TUniquePtr<IIndependentRollbackService>& Ptr : Services.IndependentRollback.Array)
	{
		Ptr->Reconcile(&VariableTickState);
	}

	bLockServices = false;
	DeferredServiceConfigDelegate.Broadcast(this);
	DeferredServiceConfigDelegate.Clear();
}

void UNetworkPredictionWorldManager::BeginNewSimulationFrame(UWorld* InWorld, ELevelTick InLevelTick, float DeltaTimeSeconds)
{
	if (InWorld != GetWorld() || !InWorld->HasBegunPlay())
	{
		return;
	}

	BeginNewSimulationFrame_Internal(DeltaTimeSeconds);
}

void UNetworkPredictionWorldManager::BeginNewSimulationFrame_Internal(float DeltaTimeSeconds)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NP_TICK);
	TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::Tick);

	ActiveInstance = this;
	bLockServices = true;

	const float fEngineFrameDeltaTimeMS = DeltaTimeSeconds * 1000.f;

	// -------------------------------------------------------------------------
	//	Fixed Tick
	// -------------------------------------------------------------------------
	if (Services.FixedTick.Array.Num() > 0)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_NP_TICK_FIXED);
		TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::FixedTick);

		FixedTickState.UnspentTimeMS += fEngineFrameDeltaTimeMS;

		while ((FixedTickState.UnspentTimeMS + KINDA_SMALL_NUMBER) >= FixedTickState.FixedStepDilatedRealTimeMS)
		{
			FixedTickState.UnspentTimeMS -= FixedTickState.FixedStepDilatedRealTimeMS;
			if (FMath::IsNearlyZero(FixedTickState.UnspentTimeMS))
			{
				FixedTickState.UnspentTimeMS = 0.f;
			}

			FNetSimTimeStep Step = FixedTickState.GetNextTimeStep();
			FServiceTimeStep ServiceStep = FixedTickState.GetNextServiceTimeStep();

			const int32 ServerInputFrame = FixedTickState.PendingFrame + FixedTickState.Offset;

			UE_NP_TRACE_PUSH_INPUT_FRAME(ServerInputFrame);
			for (TUniquePtr<IInputService>& Ptr : Services.FixedInputRemote.Array)
			{
				Ptr->ProduceInput(FixedTickState.FixedStepMS);
			}

			for (TUniquePtr<IInputService>& Ptr : Services.FixedInputLocal.Array)
			{
				Ptr->ProduceInput(FixedTickState.FixedStepMS);
			}

			UE_NP_TRACE_PUSH_TICK(Step.TotalSimulationTime, FixedTickState.FixedStepMS, Step.Frame);

			// Should we increment PendingFrame before or after the tick?
			// Before: sims that are spawned during Tick (of other sims) will not be ticked this frame.
			// So we want their seed state/cached pending frame to be set to the next pending frame, not this one.
			FixedTickState.PendingFrame++;

			for (TUniquePtr<ILocalTickService>& Ptr : Services.FixedTick.Array)
			{
				Ptr->Tick(Step, ServiceStep);
			}

			if (Settings.bEnableFixedTickSmoothing)
			{
				for (TUniquePtr<IFixedSmoothingService>& Ptr : Services.FixedSmoothing.Array)
				{
					Ptr->UpdateSmoothing(ServiceStep, &FixedTickState);
				}
			}
		}
	}

	// -------------------------------------------------------------------------
	//	Local Independent Tick
	// -------------------------------------------------------------------------
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::IndependentTick);

		// Update VariableTickState
		constexpr int32 MinStepMS = 1;
		constexpr int32 MaxStepMS = 100;

		VariableTickState.UnspentTimeMS += fEngineFrameDeltaTimeMS;
		float fDeltaMS = FMath::FloorToFloat(VariableTickState.UnspentTimeMS);
		VariableTickState.UnspentTimeMS -= fDeltaMS;

		const int32 DeltaSimMS = FMath::Clamp((int32)fDeltaMS, MinStepMS, MaxStepMS);

		FVariableTickState::FFrame& PendingFrameData = VariableTickState.Frames[VariableTickState.PendingFrame];
		PendingFrameData.DeltaMS = DeltaSimMS;

		// Input
		UE_NP_TRACE_PUSH_INPUT_FRAME(VariableTickState.PendingFrame);
		for (TUniquePtr<IInputService>& Ptr : Services.IndependentLocalInput.Array)
		{
			Ptr->ProduceInput(DeltaSimMS);
		}

		// -------------------------------------------------------------------------
		// LocalTick
		// -------------------------------------------------------------------------

		FNetSimTimeStep Step = VariableTickState.GetNextTimeStep(PendingFrameData);
		FServiceTimeStep ServiceStep = VariableTickState.GetNextServiceTimeStep(PendingFrameData);
		UE_NP_TRACE_PUSH_TICK(Step.TotalSimulationTime, Step.StepMS, Step.Frame);

		for (TUniquePtr<ILocalTickService>& Ptr : Services.IndependentLocalTick.Array)
		{
			Ptr->Tick(Step, ServiceStep);
		}

		// -------------------------------------------------------------------------
		//	Remote Independent Tick
		// -------------------------------------------------------------------------
		for (TUniquePtr<IRemoteIndependentTickService>& Ptr : Services.IndependentRemoteTick.Array)
		{
			Ptr->Tick(DeltaTimeSeconds, &VariableTickState);
		}

		// Increment local PendingFrame and set (next) pending frame's TotalMS
		const int32 EndTotalSimTimeMS = PendingFrameData.TotalMS + PendingFrameData.DeltaMS;
		VariableTickState.Frames[++VariableTickState.PendingFrame].TotalMS = EndTotalSimTimeMS;
	}

	// -------------------------------------------------------------------------
	// Interpolation
	// -------------------------------------------------------------------------
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::Interpolation);

		if (Services.FixedInterpolate.Array.Num() > 0)
		{
			const int32 LatestRecvFrame = FMath::Max(FixedTickState.Interpolation.LatestRecvFrameAP, FixedTickState.Interpolation.LatestRecvFrameSP);
			if (LatestRecvFrame != INDEX_NONE)
			{
				// We want 100ms of buffered time. As long a actors replicate at >= 10hz, this is should be good
				// Its better to keep this simple with a single time rather than trying to coordinate lowest amount of buffered time
				// between all the registered instances in the different ModelDefs
				const int32 DesiredBufferedMS = Settings.FixedTickInterpolationBufferedMS;

				float InterpolateRate = 1.f;
				if (FixedTickState.Interpolation.ToFrame == INDEX_NONE)
				{
					const int32 NumBufferedFrames = LatestRecvFrame;
					const int32 BufferedMS = NumBufferedFrames * FixedTickState.FixedStepMS;

					//UE_LOGF(LogTemp, Warning, "BufferedMS: %d Frames: %d (No ToFrame)", BufferedMS, NumBufferedFrames);

					if (BufferedMS < DesiredBufferedMS)
					{
						// Not enough time buffered yet to start interpolating
						InterpolateRate = 0.f;
					}
					else
					{
						// Begin interpolation
						const int32 DesiredNumBufferedFrames = (DesiredBufferedMS / FixedTickState.FixedStepMS);
						FixedTickState.Interpolation.ToFrame = LatestRecvFrame - DesiredNumBufferedFrames;

						FixedTickState.Interpolation.PCT = 0.f;
						FixedTickState.Interpolation.AccumulatedTimeMS = 0.f;

						// We need to force a reconcile here since we suppress the call until interpolation starts
						for (TUniquePtr<IFixedInterpolateService>& Ptr : Services.FixedInterpolate.Array)
						{
							Ptr->Reconcile(&FixedTickState);
						}
					}
				}
				else
				{
					const int32 NumBufferedFrames = LatestRecvFrame - FixedTickState.Interpolation.ToFrame;
					const int32 BufferedMS = NumBufferedFrames * FixedTickState.FixedStepMS;

					//UE_LOGF(LogTemp, Warning, "BufferedMS: %d Frames: %d", BufferedMS, NumBufferedFrames);

					if (NumBufferedFrames <= 0)
					{
						InterpolateRate = 0.f;
					}
				}

				int32 AdvanceFrames = 0;
				if (InterpolateRate > 0.f)
				{
					const float fScaledDeltaTimeMS = (InterpolateRate * fEngineFrameDeltaTimeMS);

					FixedTickState.Interpolation.AccumulatedTimeMS += fScaledDeltaTimeMS;
					AdvanceFrames = (int32)FixedTickState.Interpolation.AccumulatedTimeMS / FixedTickState.FixedStepRealTimeMS;
					if (AdvanceFrames > 0)
					{
						FixedTickState.Interpolation.ToFrame += AdvanceFrames;
						FixedTickState.Interpolation.AccumulatedTimeMS -= (AdvanceFrames * FixedTickState.FixedStepRealTimeMS);
					}
					const float RawPCT = FixedTickState.Interpolation.AccumulatedTimeMS / (float)FixedTickState.FixedStepRealTimeMS;
					FixedTickState.Interpolation.PCT = FMath::Clamp<float>(RawPCT, 0.f, 1.f);
					npEnsureMsgf(FixedTickState.Interpolation.PCT >= 0.f && FixedTickState.Interpolation.PCT <= 1.f, TEXT("Interpolation PCT out of range. %f"), FixedTickState.Interpolation.PCT);

					const float PCTms = FixedTickState.Interpolation.PCT * (float)FixedTickState.FixedStepMS;
					FixedTickState.Interpolation.InterpolatedTimeMS = ((FixedTickState.Interpolation.ToFrame-1) * FixedTickState.FixedStepMS) + (int32)PCTms;

					//UE_LOGF(LogTemp, Warning, "[Interpolate] %ls Interpolating ToFrame %d. PCT: %.2f. Buffered: %d", *GetPathName(), FixedTickState.Interpolation.ToFrame, FixedTickState.Interpolation.PCT, FixedTickState.Interpolation.LatestRecvFrame - FixedTickState.Interpolation.ToFrame);

					{
						TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::FinalizeFrame);
						for (TUniquePtr<IFixedInterpolateService>& Ptr : Services.FixedInterpolate.Array)
						{
							Ptr->FinalizeFrame(DeltaTimeSeconds, &FixedTickState);
						}
					}
				}
			}
		}

		if (Services.IndependentInterpolate.Array.Num() > 0)
		{
			const int32 DesiredBufferedMS = Settings.IndependentTickInterpolationBufferedMS;
			const int32 MaxBufferedMS = Settings.IndependentTickInterpolationMaxBufferedMS;

			if (VariableTickState.Interpolation.LatestRecvTimeMS > DesiredBufferedMS)
			{
				float InterpolationRate = 1.f;

				const int32 BufferedMS = VariableTickState.Interpolation.LatestRecvTimeMS - (int32)VariableTickState.Interpolation.fTimeMS;
				if (BufferedMS > MaxBufferedMS)
				{
					UE_LOGF(LogNetworkPrediction, Warning, "Independent Interpolation fell behind. BufferedMS: %d", BufferedMS);
					VariableTickState.Interpolation.fTimeMS = (float)(VariableTickState.Interpolation.LatestRecvTimeMS - DesiredBufferedMS);
				}
				else if (BufferedMS <= 0)
				{
					UE_LOGF(LogNetworkPrediction, Warning, "Independent Interpolation starved: %d", BufferedMS);
					InterpolationRate = 0.f;
				}

				if (InterpolationRate > 0.f)
				{
					const float fScaledDeltaTimeMS = (InterpolationRate * fEngineFrameDeltaTimeMS);
					VariableTickState.Interpolation.fTimeMS += fScaledDeltaTimeMS;
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::FinalizeFrame);
					for (TUniquePtr<IIndependentInterpolateService>& Ptr : Services.IndependentInterpolate.Array)
					{
						Ptr->FinalizeFrame(DeltaTimeSeconds, &VariableTickState);
					}
				}
			}
		}
	}


	//-------------------------------------------------------------------------------------------------------------
	// Handle newly spawned services right now, so that they can Finalize/SendRPCs on the very first frame of life
	//-------------------------------------------------------------------------------------------------------------

	bLockServices = false;
	DeferredServiceConfigDelegate.Broadcast(this);
	DeferredServiceConfigDelegate.Clear();

	// -------------------------------------------------------------------------
	//	Finalize
	// -------------------------------------------------------------------------
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::FinalizeFrame);

		const int32 FixedTotalSimTimeMS = FixedTickState.GetTotalSimTimeMS();
		const int32 FixedServerFrame = FixedTickState.PendingFrame + FixedTickState.Offset;
		for (TUniquePtr<IFinalizeService>& Ptr : Services.FixedFinalize.Array)
		{
			Ptr->FinalizeFrame(DeltaTimeSeconds, FixedServerFrame, FixedTotalSimTimeMS, FixedTickState.FixedStepMS);
		}

		if (Settings.bEnableFixedTickSmoothing)
		{
			for (TUniquePtr<IFixedSmoothingService>& Ptr : Services.FixedSmoothing.Array)
			{
				Ptr->FinalizeSmoothingFrame(&FixedTickState);
			}
		}

		const int32 IndependentTotalSimTimeMS = VariableTickState.Frames[VariableTickState.PendingFrame].TotalMS;
		const int32 IndependentFrame = VariableTickState.PendingFrame;
		for (TUniquePtr<IFinalizeService>& Ptr : Services.IndependentLocalFinalize.Array)
		{
			Ptr->FinalizeFrame(DeltaTimeSeconds, IndependentFrame, IndependentTotalSimTimeMS, 0);
		}

		for (TUniquePtr<IRemoteFinalizeService>& Ptr : Services.IndependentRemoteFinalize.Array)
		{
			Ptr->FinalizeFrame(DeltaTimeSeconds);
		}
	}

	// -------------------------------------------------------------------------
	// Call server RPC (common)
	// -------------------------------------------------------------------------
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::CallServerRPC);
		for (TUniquePtr<IServerRPCService>& Ptr : Services.ServerRPC.Array)
		{
			Ptr->CallServerRPC(DeltaTimeSeconds);
		}
	}
}

ENetworkPredictionTickingPolicy UNetworkPredictionWorldManager::PreferredDefaultTickingPolicy() const
{
	return Settings.PreferredTickingPolicy;
}

