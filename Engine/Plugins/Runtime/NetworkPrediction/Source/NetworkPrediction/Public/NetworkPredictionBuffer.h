// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Array.h"
#include "Templates/Function.h"

// Helper struct to encapsulate optional, delayed writing of new element to TNetworkSimAuxBuffer buffer
template<typename ElementType>
struct TNetSimLazyWriterFunc
{
	template<typename TBuffer>
	TNetSimLazyWriterFunc(TBuffer& Buffer, int32 PendingFrame)
	{
		GetFunc = [&Buffer, PendingFrame]() { return Buffer.WriteAtFrame(PendingFrame); };
	}

	// Hack
	TNetSimLazyWriterFunc(void* Data)
	{
		GetFunc = [Data]() { return Data; };
	}

	ElementType* Get() const
	{
		check(GetFunc);
		return (ElementType*)GetFunc();
	}

	TFunction<void*()> GetFunc;
};

// Reference version of LazyWriter. This is what gets passed through chains of ::SimulationTick calls. This is to avoid copying the TFunction in TNetSimLazyWriterFunc around.
template<typename ElementType>
struct TNetSimLazyWriter
{
	template<typename TLazyWriter>
	TNetSimLazyWriter(const TLazyWriter& Parent)
		: GetFunc(Parent.GetFunc) { }

	ElementType* Get() const
	{
		return (ElementType*)GetFunc();
	}

	TFunctionRef<void*()> GetFunc;
};

// Sparse buffer: each element has explicit Frame value
template<typename ElementType> 
struct TNetworkSimAuxBuffer
{
	TNetworkSimAuxBuffer(int32 Capacity)
	{
		Init(Capacity);
	}

	ElementType* operator[](int32 Frame)
	{
		return const_cast<ElementType*>(GetImpl(Frame)); 
	}
	
	const ElementType* operator[](int32 Frame) const
	{
		return GetImpl(Frame); 
	}

	// Writes a new element at Frame. Copies previous frames contents into new element.
	ElementType* WriteAtFrame(int32 Frame)
	{
		const int32 TailPos = FMath::Max(0, HeadPosition - Elements.Num() + 1);
		int32 Pos = HeadPosition;
		do
		{
			TInternal& Data = Elements[Pos & IndexMask];
			if (Data.Frame <= Frame)
			{
				HeadPosition = Pos+1;
				TInternal& NewData = Elements[HeadPosition & IndexMask];
				NewData.Frame = Frame;
				NewData.Element = Data.Element;
				return &NewData.Element;
			}
			Pos--;
		} while(--Pos >= TailPos);

		HeadPosition = 0;
		TInternal& NewData = Elements[0];
		NewData.Element = ElementType();
		NewData.Frame = Frame;
		return &NewData.Element;
	}

	TNetSimLazyWriterFunc<ElementType> LazyWriter(int32 Frame)
	{
		return TNetSimLazyWriterFunc<ElementType>(*this, Frame);
	}

	// Resizes while preserving contents from head position
	void Resize(int32 NewSize)
	{
		check(NewSize > 0);
		TArray<TInternal> PrevElements = MoveTemp(Elements);
		int32 PrevMask = IndexMask;

		Init(NewSize);

		int32 StartCopyFrom = HeadPosition - FMath::Min(PrevElements.Num(), NewSize) + 1;
		for (int32 i=StartCopyFrom; i <= HeadPosition; ++i)
		{
			Elements[i & IndexMask] = PrevElements[i & PrevMask];
		}
	}

private:

	struct TInternal
	{
		int32 Frame = -1;
		ElementType Element;
	};

	void Init(int32 NewSize)
	{
		Elements.Reset();
		Elements.AddDefaulted(FMath::RoundUpToPowerOfTwo(NewSize));
		IndexMask = Elements.Num() - 1;	
	}

	const ElementType* GetImpl(int32 Frame) const
	{
		const int32 TailPos = FMath::Max(0, HeadPosition - Elements.Num() + 1);
		int32 Pos = HeadPosition;
		do
		{
			const TInternal& Data = Elements[Pos & IndexMask];
			if (Data.Frame <= Frame)
			{
				return &Data.Element;
			}
		} while(--Pos >= TailPos);

		return nullptr;
	}

	// Latest element to be written
	int32 HeadPosition = 0;

	// Holds the mask for indexing the buffer's elements
	int32 IndexMask = 0;

	// Holds the buffer's elements
	TArray<TInternal> Elements;
};

// Circular buffer for storing continuous frame data
template<typename ElementType, bool SingleElement=false> 
struct TNetworkPredictionBuffer
{
	TNetworkPredictionBuffer(int32 Capacity=2)
	{
		check(Capacity > 0);
		Init(Capacity);
	}

	TNetworkPredictionBuffer(const TNetworkPredictionBuffer<ElementType, SingleElement>&& Other)
		: IndexMask(Other.IndexMask)
		, Elements(MoveTemp(Other.Elements))
	{
	}

	TNetworkPredictionBuffer<ElementType, SingleElement>& operator=(TNetworkPredictionBuffer<ElementType, SingleElement>&& Other)
	{
		IndexMask = Other.IndexMask;
		Elements = MoveTemp(Other.Elements);
		return *this;
	}
	
	ElementType& operator[](int32 Index)
	{
		return Elements[Index & IndexMask];
	}
	
	const ElementType& operator[](int32 Index) const
	{
		return Elements[Index & IndexMask];
	}
	
	int32 Capacity() const
	{
		return Elements.Num();
	}

	// Resizes while preserving contents from head position
	void Resize(int32 NewSize, int32 PrevHead)
	{
		check(NewSize > 0);
		TArray<ElementType> PrevElements = MoveTemp(Elements);
		int32 PrevMask = IndexMask;

		Init(NewSize);

		int32 StartCopyFrom = PrevHead - FMath::Min(PrevElements.Num(), NewSize) + 1;
		for (int32 i=StartCopyFrom; i <= PrevHead; ++i)
		{
			Elements[i & IndexMask] = PrevElements[i & PrevMask];
		}
	}

private:

	void Init(int32 NewSize)
	{
		Elements.Reset();
		Elements.AddDefaulted(FMath::RoundUpToPowerOfTwo(NewSize));
		IndexMask = Elements.Num() - 1;	
	}

	// Holds the mask for indexing the buffer's elements.
	int32 IndexMask = 0;

	// Holds the buffer's elements.
	TArray<ElementType> Elements;
};

template<typename ElementType> 
struct TNetworkPredictionBuffer<ElementType, true>
{
	ElementType& operator[](int32 Index) { return Element; }
	
	const ElementType& operator[](int32 Index) const { return Element; }
	int32 Capacity() const { return 1; }
	void Resize(int32 NewSize, int32 PrevHead) { }

private:
	ElementType Element;
};
