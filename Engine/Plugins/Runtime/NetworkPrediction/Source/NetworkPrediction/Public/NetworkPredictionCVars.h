// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/IConsoleManager.h"

// ------------------------------------------------------------------------------------------------------------
//	"Shipping const" cvars: cvars that should compile out to const functions in shipping/test builds
//		This got a little tricky due to templated usage across different modules. Previous patterns for
//		this thing didn't work. This implementation requires manual finding of consolve variables so will be 
//		a bit slower but shouldn't matter since it is compiled out of shipping/test.
// ------------------------------------------------------------------------------------------------------------

inline IConsoleVariable* FindConsoleVarHelper(const TCHAR* VarName)
{
	return IConsoleManager::Get().FindConsoleVariable(VarName, false);
}

// Whether to treat these cvars as consts
#ifndef NETSIM_CONST_CVARS
	#define NETSIM_CONST_CVARS (UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

// This is required because these cvars live in header files that will be included across different compilation units
// Just using standard FAutoConsoleVariableRef will cause multiple registrations of the same variable
struct FConditionalAutoConsoleRegister
{
	FConditionalAutoConsoleRegister(const TCHAR* VarName, int32 Value, const TCHAR* Help)
	{
		if (!IConsoleManager::Get().FindConsoleVariable(VarName, false))
		{
			IConsoleManager::Get().RegisterConsoleVariable(VarName, Value, Help, ECVF_Cheat);
		}
	}

	FConditionalAutoConsoleRegister(const TCHAR* VarName, float Value, const TCHAR* Help)
	{
		if (!IConsoleManager::Get().FindConsoleVariable(VarName, false))
		{
			IConsoleManager::Get().RegisterConsoleVariable(VarName, Value, Help, ECVF_Cheat);
		}
	}
};

#define NP_DEVCVAR_INT(Var,Value,VarName,Help) \
	int32 Var = Value; \
	static FAutoConsoleVariableRef Var##CVar(TEXT(VarName), Var, TEXT(Help), ECVF_Cheat );

#define NETSIM_DEVCVAR_INT(Var,Value,VarName,Help) \
	static FConditionalAutoConsoleRegister Var##Auto(TEXT(VarName),(int32)Value,TEXT(Help)); \
	inline int32 Var() \
	{ \
		static const auto* Existing = FindConsoleVarHelper(TEXT(VarName)); \
		check(Existing); \
		return Existing->GetInt(); \
	} \
	inline void Set##Var(int32 _V) \
	{ \
		static auto* Existing = FindConsoleVarHelper(TEXT(VarName)); \
		check(Existing); \
		Existing->Set(_V, ECVF_SetByConsole); \
	}

#if NETSIM_CONST_CVARS
#define NETSIM_DEVCVAR_SHIPCONST_INT(Var,Value,VarName,Help) \
	inline int32 Var() { return Value; } \
	inline void Set##Var(int32 _V) { }
#else
#define NETSIM_DEVCVAR_SHIPCONST_INT(Var,Value,VarName,Help) NETSIM_DEVCVAR_INT(Var,Value,VarName,Help)
#endif



#define NP_DEVCVAR_FLOAT(Var,Value,VarName,Help) \
	float Var = Value; \
	static FAutoConsoleVariableRef Var##CVar(TEXT(VarName), Var, TEXT(Help), ECVF_Cheat );

#define NETSIM_DEVCVAR_FLOAT(Var,Value,VarName,Help) \
	static FConditionalAutoConsoleRegister Var##Auto(TEXT(VarName),(float)Value,TEXT(Help)); \
	inline float Var() \
	{ \
		static const auto* Existing = FindConsoleVarHelper(TEXT(VarName)); \
		check(Existing); \
		return Existing->GetFloat(); \
	} \
	inline void Set##Var(float _V) \
	{ \
		static auto* Existing = FindConsoleVarHelper(TEXT(VarName)); \
		check(Existing); \
		Existing->Set(_V, ECVF_SetByConsole); \
	}

#if NETSIM_CONST_CVARS
#define NETSIM_DEVCVAR_SHIPCONST_FLOAT(Var,Value,VarName,Help) \
	inline float Var() { return Value; } \
	inline void Set##Var(float _V) { }
#else
#define NETSIM_DEVCVAR_SHIPCONST_FLOAT(Var,Value,VarName,Help) NETSIM_DEVCVAR_FLOAT(Var,Value,VarName,Help)
#endif

