// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#ifndef NP_ENSURES_ALWAYS
#define NP_ENSURES_ALWAYS 0 
#endif

#define NP_CHECKS_AND_ENSURES 1
#if NP_CHECKS_AND_ENSURES
	#define npCheck(Condition) check(Condition)
	#define npCheckf(Condition, ...) checkf(Condition, ##__VA_ARGS__)
	#if NP_ENSURES_ALWAYS
		#define npEnsure(Condition) ensureAlways(Condition)
		#define npEnsureMsgf(Condition, ...) ensureAlwaysMsgf(Condition, ##__VA_ARGS__)
	#else
		#define npEnsure(Condition) ensure(Condition)
		#define npEnsureMsgf(Condition, ...) ensureMsgf(Condition, ##__VA_ARGS__)
	#endif
#else
	#define npCheck(...)
	#define npCheckf(...)
	#define npEnsure(Condition) (!!(Condition))
	#define npEnsureMsgf(Condition, ...) (!!(Condition))
#endif

#define NP_CHECKS_AND_ENSURES_SLOW !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#if NP_CHECKS_AND_ENSURES_SLOW
	#define npCheckSlow(Condition) check(Condition)
	#define npCheckfSlow(Condition, ...) checkf(Condition, ##__VA_ARGS__)
	#if NP_ENSURES_ALWAYS
		#define npEnsureSlow(Condition) ensureAlways(Condition)
		#define npEnsureMsgfSlow(Condition, ...) ensureAlwaysMsgf(Condition, ##__VA_ARGS__)
	#else
		#define npEnsureSlow(Condition) ensure(Condition)
		#define npEnsureMsgfSlow(Condition, ...) ensureMsgf(Condition, ##__VA_ARGS__)
	#endif
#else
	#define npCheckSlow(Condition)
	#define npCheckfSlow(...)
	#define npEnsureSlow(Condition) (!!(Condition))
	#define npEnsureMsgfSlow(Condition, ...) (!!(Condition))
#endif