// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Components/ActorComponent.h"
#include "NetworkPredictionProxy.h"
#include "NetworkPredictionReplicationProxy.h"

#include "NetworkPredictionComponent.generated.h"

#define UE_API NETWORKPREDICTION_API

enum ENetRole : int;
namespace EEndPlayReason { enum Type : int; }

// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------
//	UNetworkPredictionComponent
//	This is the base component for running a TNetworkedSimulationModel through an actor component. This contains the boiler plate hooks into getting the system
//	initialized and plugged into the UE replication system.
//
//	This is an abstract component and cannot function on its own. It must be subclassed and InitializeNetworkPredictionProxy must be implemented.
//	Ticking and RPC sending will be handled automatically.
//
//	Its also worth pointing out that nothing about being a UActorComponent is essential here. All that this component does could be done within an AActor itself.
//	An actor component makes sense for flexible/reusable code provided by the plugin. But there is nothing stopping you from copying this directly into an actor if you had reason to.
// ----------------------------------------------------------------------------------------------------------------------------------------------------------------------
UCLASS(MinimalAPI, Abstract)
class UNetworkPredictionComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	UE_API UNetworkPredictionComponent();

	UE_API virtual void InitializeComponent() override;
	UE_API virtual void PreReplication(IRepChangedPropertyTracker& ChangedPropertyTracker) override;
	UE_API virtual void PreNetReceive() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type Reason);

	// Invoke the ServerRPC, called from UNetworkPredictionWorldManager via the TServerRPCService.
	UE_API void CallServerRPC();

protected:
	
	// Classes must initialize the NetworkPredictionProxy (register with the NetworkPredictionSystem) here. EndPlay will unregister.
	virtual void InitializeNetworkPredictionProxy() { check(false); }

	// Finalizes initialization when NetworkRole changes. Does not need to be overridden.
	UE_API virtual void InitializeForNetworkRole(ENetRole Role, const bool bHasNetConnection);

	// Helper: Checks if the owner's role has changed and calls InitializeForNetworkRole if necessary.
	UE_API bool CheckOwnerRoleChange();

	// The actual ServerRPC. This needs to be a UFUNCTION but is generic and not dependent on the simulation instance
	UFUNCTION(Server, unreliable, WithValidation)
	UE_API void ServerReceiveClientInput(const FServerReplicationRPCParameter& ProxyParameter);

	// Proxy to interface with the NetworkPrediction system
	UPROPERTY(Replicated, transient)
	FNetworkPredictionProxy NetworkPredictionProxy;

	// ReplicationProxies are just pointers to the data/NetSerialize functions within the NetworkSim
	UPROPERTY()
	FReplicationProxy ReplicationProxy_ServerRPC;

private:

	UPROPERTY(Replicated, transient)
	FReplicationProxy ReplicationProxy_Autonomous;

	UPROPERTY(Replicated, transient)
	FReplicationProxy ReplicationProxy_Simulated;

	UPROPERTY(Replicated, transient)
	FReplicationProxy ReplicationProxy_Replay;

	/** Iris support: used for writing server RPC parameter bits. */
	FNetBitWriter ServerRPCParamBitWriter;

	/** Iris support: used for reading server RPC parameter bits. */
	FNetBitReader ServerRPCParamBitReader;


protected:

	FReplicationProxySet GetReplicationProxies()
	{
		return FReplicationProxySet{ &ReplicationProxy_ServerRPC, &ReplicationProxy_Autonomous, &ReplicationProxy_Simulated, &ReplicationProxy_Replay };
	}
};

#undef UE_API
