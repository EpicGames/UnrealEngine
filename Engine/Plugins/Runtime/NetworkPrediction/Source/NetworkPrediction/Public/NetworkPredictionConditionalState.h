// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NetworkPredictionCheck.h"

template<typename TestType, typename UnderlyingType=TestType>
struct TConditionalState
{
	enum { Valid = true };

	operator UnderlyingType*() { return &State; }

	const UnderlyingType* operator->() const { return &State; }
	UnderlyingType* operator->() { return &State; }

	UnderlyingType* Get() { return &State; }
	const UnderlyingType* Get() const { return &State; }

	void CopyTo(UnderlyingType* Dest) const
	{
		npCheckSlow(Dest);
		*Dest = State;
	}

private:
	UnderlyingType State;
};

template<typename UnderlyingType>
struct TConditionalState<void, UnderlyingType>
{
	enum { Valid = false };

	operator void*() const { return nullptr; }

	const void* operator->() const { return nullptr; }
	void* operator->() { return nullptr; }

	void* Get() { return nullptr; }
	const void* Get() const { return nullptr; }

	void CopyTo(void* Dest) const { }
};