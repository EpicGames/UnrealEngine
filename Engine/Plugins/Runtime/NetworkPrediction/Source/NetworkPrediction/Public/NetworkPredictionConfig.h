// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionConfig.generated.h"

// Must be kept in sync with ENP_TickingPolicy
UENUM()
enum class ENetworkPredictionTickingPolicy : uint8
{
	// Client ticks at local frame rate. Server ticks clients independently at client input cmd rate.
	Independent	= 1 << 0,
	// Everyone ticks at same fixed rate. Supports group rollback.
	Fixed		= 1 << 1,

	All = Independent | Fixed UMETA(Hidden),
};
ENUM_CLASS_FLAGS(ENetworkPredictionTickingPolicy);

enum class ENetworkPredictionLocalInputPolicy : uint8
{
	// Up to the user to write input via FNetSimProxy::WriteInputCmd.
	Passive,
	// ProduceInput is called on the driver before every simulation frame. This may be necessary for things like aim assist and fixed step simulations that run multiple sim frames per engine frame
	PollPerSimFrame,
};

// Must be kept in sync with ENP_NetworkLOD. Note: SimExtrapolate Not currently implemented so it is hidden
UENUM()
enum class ENetworkLOD : uint8
{
	Interpolated	= 1 << 0,
	SimExtrapolate	= 1 << 1 UMETA(Hidden),
	ForwardPredict	= 1 << 2,

	All = Interpolated | SimExtrapolate | ForwardPredict UMETA(Hidden),
};
ENUM_CLASS_FLAGS(ENetworkLOD);

static constexpr ENetworkLOD GetHighestNetworkLOD(ENetworkLOD Mask)
{
	if ((uint8)Mask >= (uint8)ENetworkLOD::ForwardPredict)
	{
		return ENetworkLOD::ForwardPredict;
	}

	if ((uint8)Mask >= (uint8)ENetworkLOD::SimExtrapolate)
	{
		return ENetworkLOD::SimExtrapolate;
	}

	return ENetworkLOD::Interpolated;
}

// -------------------------------------------------------------------------------------------------------------

// What a ModelDef of capable of
struct FNetworkPredictionModelDefCapabilities
{
	struct FSupportedNetworkLODs
	{
		ENetworkLOD	AP;
		ENetworkLOD	SP;
	};

	FSupportedNetworkLODs FixedNetworkLODs = FSupportedNetworkLODs{ ENetworkLOD::All, ENetworkLOD::All };
	FSupportedNetworkLODs IndependentNetworkLODs = FSupportedNetworkLODs{ ENetworkLOD::All, ENetworkLOD::Interpolated | ENetworkLOD::SimExtrapolate };

	ENetworkPredictionTickingPolicy SupportedTickingPolicies = ENetworkPredictionTickingPolicy::All;
};

// How a registered instance should behave globally. That is, independent of any instance state (local role, connection, significance, local budgets). E.g, everyone agrees on this.
// This can be changed explicitly by the user or simulation. For example, a sim that transitions between fixed and independent ticking modes.
struct FNetworkPredictionInstanceArchetype
{
	ENetworkPredictionTickingPolicy	TickingMode;
	void NetSerialize(FArchive& Ar)
	{
		Ar << TickingMode;
	}
};

// The config should tell us what services we should be subscribed to. See UNetworkPredictionWorldManager::ConfigureInstance
// This probably needs to be split into two parts:
//	1. What settings/config that the server is authority over and must be agreed on (TickingPolicy)
//	2. What are the local settings that can be lodded around?
struct FNetworkPredictionInstanceConfig
{
	ENetworkPredictionLocalInputPolicy InputPolicy = ENetworkPredictionLocalInputPolicy::Passive;
	ENetworkLOD NetworkLOD = ENetworkLOD::ForwardPredict;
};
