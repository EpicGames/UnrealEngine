// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "Templates/UnrealTypeTraits.h"

class FArchive;

// ----------------------------------------------------------------------------------------------------------------------------------------------
//	NetSimCue Traits: compile time settings for NetSimeCue types that determine who can invoke the event and who it replicates to.
//	See "Mock Cue Example" in NetworkPredictionCueTests.cpp for minimal examples for using/setting traits.
//
//	There are three traits:
//
//	// Who can Invoke this Cue in their simulation (if this test fails, the Invoke call is suppressed locally)
//	static constexpr ENetSimCueInvoker InvokeMask { InInvokeMask };
//
//	// Whether the cue will be invoked during resimulates (which will require cue to be rollbackable)
//	static constexpr bool Resimulate { InResimulate };
//
//	// Does the cue replicate? (from authority). This will also determine if the cue needs to be saved locally for NetIdentical tests (to avoid double playing) and if needs to support rollback
//	static constexpr ENetSimCueReplicationTarget ReplicationTarget { InReplicationTarget };
//
// ----------------------------------------------------------------------------------------------------------------------------------------------

// When we run a SimulationTick, it will be done under one of these contexts. This isn't NetSimCue specific and probably makes sense to move to the NetSimModel types
enum class ESimulationTickContext : uint8
{
	None				= 0,
	Authority			= 1 << 0,	// The authority (usually "server" but could also mean client authoritative client)
	Predict				= 1 << 1,	// Predicting client: autonomous proxy ("controlling client")
	Resimulate			= 1 << 2,	// Predicting client during resimulate (simulation rollback into resimulate steps)
	SimExtrapolate		= 1 << 3,	// Simulation extrapolation: non controlling client (simulated proxy) running the simulation to extrapolate
	ResimExtrapolate	= 1 << 4,	// Simulation extrapolation during a reconcile (sim rolled back to server state then stepped again to "catch back up")
};
ENUM_CLASS_FLAGS(ESimulationTickContext);

// High level "who can invoke this". Does not take resimulate into account yet. All combinations of this enum are valid (though Authority | SimExtrapolate is a bit weird maybe)
enum class ENetSimCueInvoker : uint8
{
	Authority			= (uint8)ESimulationTickContext::Authority,
	Predict				= (uint8)ESimulationTickContext::Predict,
	SimExtrapolate		= (uint8)ESimulationTickContext::SimExtrapolate,

	All					= Authority | Predict | SimExtrapolate
};
ENUM_CLASS_FLAGS(ENetSimCueInvoker);

// Turns "Who can invoke this" + "Does this play during resimulate" into the final ESimulationTickContext mask we use at runtime. This is done so users can't make invalid configurations (like "Authority | Resimulate")
static constexpr ESimulationTickContext GetSimTickMask(const ENetSimCueInvoker Invoker, const bool bAllowResimulate)
{
	ESimulationTickContext Mask = (ESimulationTickContext)Invoker;
	ESimulationTickContext AutoMask = (bAllowResimulate && EnumHasAllFlags(Invoker, ENetSimCueInvoker::Predict)) ? ESimulationTickContext::Resimulate : ESimulationTickContext::None;
	ESimulationTickContext SimMask = (bAllowResimulate && EnumHasAllFlags(Invoker, ENetSimCueInvoker::SimExtrapolate)) ? ESimulationTickContext::ResimExtrapolate : ESimulationTickContext::None;
	return Mask | AutoMask | SimMask;
}

// Who a NetSimCue should replicate/be accepted by
enum class ENetSimCueReplicationTarget : uint8
{
	None			= 0,		// Do not replicate cue to anyone
	AutoProxy		= 1 << 0,	// Replicate to autonomous proxies (controlling clients)
	SimulatedProxy	= 1 << 1,	// Replicate to simulated proxies that are running the simulation
	Interpolators	= 1 << 2,	// Replicate to simulated proxies that are *not* running the simulation themselves (E.,g interpolating)
	
	All = AutoProxy | SimulatedProxy | Interpolators,
};
ENUM_CLASS_FLAGS(ENetSimCueReplicationTarget);

// ----------------------------------------------------------------------------------------------------------------------------------------------
//	NetSimCue Traits Presets. These are reasonable settings that may be appropriate in common cases. Presets are provided so that individual settings
//	are not duplicated throughout the code base and to establish consistent vocabulary for the common types.
//
//	For quick reference, we expect these to be the most common:
//		NetSimCueTraits::Weak - The default trait type and cheapest cue to use. No replication or NetIdentical testing. Never rolled back. Predicted but never resimulated.
//		NetSimCueTraits::Strong - The most robust trait type. Will replicate to everyone and will rollback/resimulate. Must implement NetSerialize/NetIdentical. Most expensive.
//
//	The other presets fall somewhere in the middle and require thought/nuance to decide if they are right for your case.
//	Other configs are possible but its not clear their usefullness. For example, a cue that only plays on simulated clients.
//
// ----------------------------------------------------------------------------------------------------------------------------------------------

namespace NetSimCueTraits
{
	// Default Traits: if you do not explicitly set Traits on your class, this is what it defaults to
	using Default = struct Weak;

	// Non replicated cue that only plays during "latest" simulate. Will not be played during rewind/resimulate.
	// Lightest weight cue. Best used for cosmetic, non critical events. Footsteps, impact effects, etc.
	struct Weak
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::All };
		static constexpr bool Resimulate { false };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::None };
	};

	// Same as above but will only play on the owning, autonomous proxy client (not on authority, not on simulated clients)
	// Useful if the Cue is needed for only the controlling player, maybe like a HUD/UI notification pop
	struct WeakOwningClientOnly
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::Predict };
		static constexpr bool Resimulate { false };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::None };
	};

	// Same as above but will play on all clients, just not the authority.
	// Useful for something that is purely cosmetic. E.g, does not run on the server but all clients should see it if they are running the sim
	struct WeakClientsOnly
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::Predict | ENetSimCueInvoker::SimExtrapolate };
		static constexpr bool Resimulate { false };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::None };
	};

	// Will only play on the authority path and not replicate to anyone else
	struct AuthorityOnly
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::Authority };
		static constexpr bool Resimulate { false };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::None };
	};

	// Only invoked on authority and will replicate to everyone else. Not predicted so controlling client will see delays!
	// Best for events that are critical that cannot be rolled back/undown and do not need to be predicted.
	struct ReplicatedNonPredicted
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::Authority };
		static constexpr bool Resimulate { false };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::All };
	};

	// Replicated to interpolating proxies, predicted by autonomous/simulated proxy
	// Best for events you want everyone to see but don't need to get perfect in the predicting cases: doesn't need to rollback and cheap on cpu (no NetIdentical tests on predicted path)
	struct ReplicatedXOrPredicted
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::All };
		static constexpr bool Resimulate { false };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::Interpolators };
	};

	// Invoked and replicated to all. NetIdentical testing to avoid double playing, rollbackable so that it can (re)play during resimulates
	// Most expensive (bandwidth/CPU) and requires rollback callbacks to be implemented to be correct. But will always be shown "as correct as possible"
	struct Strong
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::All };
		static constexpr bool Resimulate { true };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::All };
	};

	// Non replicated but if a resimulate happens, the cue is undone and replayed.
	// This is not common and doesn't really have a clear use case. But the system can support it.
	struct NonReplicatedResimulated
	{
		static constexpr ENetSimCueInvoker InvokeMask { ENetSimCueInvoker::All };
		static constexpr bool Resimulate { true };
		static constexpr ENetSimCueReplicationTarget ReplicationTarget { ENetSimCueReplicationTarget::None };
	};
}

// Explicit trait settings. This can be used to explicitly set your traits without using a preset.
template<ENetSimCueInvoker InInvokeMask, ENetSimCueReplicationTarget InReplicationTarget, bool InResimulate>
struct TNetSimCueTraitsExplicit
{
	// Who can Invoke this Cue in their simulation (if this test fails, the Invoke call is suppressed locally)
	static constexpr ENetSimCueInvoker InvokeMask { InInvokeMask };

	// Whether the cue will be invoked during resimulates (which will require cue to be rollbackable)
	static constexpr bool Resimulate { InResimulate };

	// Does the cue replicate? (from authority). This will also determine if the cue needs to be saved locally for NetIdentical tests (to avoid double playing)
	static constexpr ENetSimCueReplicationTarget ReplicationTarget { InReplicationTarget };
};


// ----------------------------------------------------------------------------------------------------------------------------------------------
//	NetSimCue Traits template helpers
// ----------------------------------------------------------------------------------------------------------------------------------------------

// SFINAE helper
template<typename T>
struct TToVoid
{
	using type  = void;
};

// Helper to see if T has a Traits type or not
template<typename T, typename dummy=void>
struct THasNetSimCueTraits
{
	enum { Value = false };
};

template<typename T>
struct THasNetSimCueTraits<T, typename TToVoid<typename T::Traits>::type>
{
	enum { Value = true };
};

// Selects explicit traits set by T
template<typename T, bool HasTraits = THasNetSimCueTraits<T>::Value>
struct TSelectNetSimCueTraits
{
	using Traits = typename T::Traits;
};

// Fall back to Default traits
template<typename T>
struct TSelectNetSimCueTraits<T, false>
{
	using Traits = NetSimCueTraits::Default;
};

// Actual trait struct that we use to look up traits. The ways this can be set:
//	1. Explicitly specialize TNetSimCueTraits for your type (non intrusive, but does not inherit)
//	2. Explicitly set Traits inside your struct. E.g:
//			using Traits = TNetSimCueTraits_Strong; (intrustive but more concise, does support inherited types)
//	3. Automatically falls back to NetSimCueTraits::Default if not explicitly set above
template<typename TCue>
struct TNetSimCueTraits
{
	using Traits = typename TSelectNetSimCueTraits<TCue>::Traits;

	static constexpr ENetSimCueInvoker InvokeMask { Traits::InvokeMask };
	static constexpr bool Resimulate { Traits::Resimulate };
	static constexpr ENetSimCueReplicationTarget ReplicationTarget { Traits::ReplicationTarget };
	static constexpr ESimulationTickContext SimTickMask() { return GetSimTickMask((ENetSimCueInvoker)InvokeMask, Resimulate); };
};

// Type requirements: helper to determine if NetSerialize/NetIdentical functions need to be defined for user types based on the above traits
template <typename TCue>
struct TNetSimCueTypeRequirements
{
	enum
	{
		// NetSerialize is required if we ever need to replicate
		RequiresNetSerialize = (TNetSimCueTraits<TCue>::ReplicationTarget != ENetSimCueReplicationTarget::None),
		// Likewise for NetIdentical, but also if we plan to invoke during Resimulate too (even if non replicated, we use NetIdentical for comparisons. though this is probably a non practical use case).
		RequiresNetIdentical = (TNetSimCueTraits<TCue>::ReplicationTarget != ENetSimCueReplicationTarget::None) || (TNetSimCueTraits<TCue>::Resimulate)
	};
};

// ----------------------------------------------------------------------------------------------------------------------------------------------
//	Function checking helpers - helps us do clean checks for member function (NetSerialize/NetIdentical) when registering types
// ----------------------------------------------------------------------------------------------------------------------------------------------

// Helper to compile time check if NetSerialize exists
GENERATE_MEMBER_FUNCTION_CHECK(NetSerialize, void,, FArchive&);

// Helper to compile time check if NetIdentical exists (since argument is template type, must be wrapped in helper struct)
template<typename TCue>
struct THasNetIdenticalHelper
{
	GENERATE_MEMBER_FUNCTION_CHECK(NetIdentical, bool, const, const TCue&);
	enum { Value = THasMemberFunction_NetIdentical<TCue>::Value };
};

// Helper to call NetIdentical if type defines it
template<typename TCue, bool Enabled=THasNetIdenticalHelper<TCue>::Value>
struct TNetCueNetIdenticalHelper
{
	static bool CallNetIdenticalOrNot(const TCue& Cue, const TCue& Other) { ensure(false); return false; } // This should never be hit by cue types that don't need to NetIdentical
};

template<typename TCue>
struct TNetCueNetIdenticalHelper<TCue, true>
{
	static bool CallNetIdenticalOrNot(const TCue& Cue, const TCue& Other) { return Cue.NetIdentical(Other); }
};

// Helper to call NetSerialize if type defines it
template<typename TCue, bool Enabled=THasMemberFunction_NetSerialize<TCue>::Value>
struct TNetCueNetSerializeHelper
{
	static void CallNetSerializeOrNot(TCue& Cue, FArchive& Ar) { ensure(false); } // This should never be hit by cue types that don't need to NetSerialize
};

template<typename TCue>
struct TNetCueNetSerializeHelper<TCue, true>
{
	static void CallNetSerializeOrNot(TCue& Cue, FArchive& Ar) { Cue.NetSerialize(Ar); }
};
