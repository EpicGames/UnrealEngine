// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "NetworkPredictionCueTraits.h"
#include "NetworkPredictionCheck.h"
#include "Delegates/Delegate.h"

/*=============================================================================
Networked Simulation Cues

Cues are events that are outputted by a Networked Simulation. They have the following properties:
-Cues are defined by a user struct (custom data payload). They are *invoked* during a SimulationTick and are *dispatched* to a *handler*.
-They are dispatched *after* the simulation is done ticking (via TNetSimCueDispatcher::DispatchCueRecord which is called during the owning component's tick).
-They should not affect the simulation. Or rather, if they do affect the simulation, it will be during the actor tick, effectively the same as any "out of band" modifications.
-They provide automatic replication and invocation settings (traits). They will not "double play" during resimulates.
-They are time aware. When dispatched, the receiver is given how much time has passed (relative to local "head" time) since the invocation.
-They are rollback aware. When dispatched, the receiver is given a callback that will be invoked if the cue needs to rollback (undo itself).
-The callback is not given in contexts when a rollback is impossible: e.g, on the authority.

Notes on reliability:
-Cues are unreliable in nature. If you join a game in progress, an actor driving the sim suddenly becomes relevant: you will not get all past events.
-If network becomes saturated or drops, you may miss events too.
-Order of received cues is guaranteed, but we can't promise there won't be gaps/missing cues!
-In other words: do not use cues in stateful ways! Cues should be used for transient events ("NotifyExplosion") not state ("NotifySetDestroyed(true)")
-For state transitions, just use FinalizeFrame to detect these changes. "State transitions" are 100% reliable, but (obviously) cannot use data that is not in the Sync/Aux buffer.

Notes on "simulation affecting events" (E.g, *not* NetSimCues)
-If you have an event that needs to affect the simulation during the simulation - that is seen as an extension of the simulation and is "up to the user" to implement.
-In other words, handle the event yourself inline with the simulation. That means directly broadcasting/calling other functions/into other classes/etc inside your simulation.
-If your event has state mutation on the handler side, that is a hazard (e.g, state that the network sim is not aware of, but is used in the event code which is an extension of the simulation)
-In these cases I would recommend: A) on the handler side, don't write to any non-networked-sim state if non authority or B) just don't handle the event on non authority. (Expect corrections)
-If the handler side doesn't have state hazards, say a teleporting volume that always does the same thing: there is no reason everyone can't run the event. Its an extension of the simulation.

See "Mock Cue Example" in NetworkPredictionCueTests.cpp for minimal example of implementing the Cue types and Handler classes.

=============================================================================*/

NETWORKPREDICTION_API DECLARE_LOG_CATEGORY_EXTERN(LogNetworkPredictionCues, Display, All);

#ifndef NETSIMCUE_TYPEID_TYPE
#define NETSIMCUE_TYPEID_TYPE uint8;
#endif

using FNetSimCueTypeId = NETSIMCUE_TYPEID_TYPE;

template<typename ModelDef>
struct FNetworkPredictionDriver;

// ----------------------------------------------------------------------------------------------------------------------------------------------
//	Wrapper: wraps the actual user NetSimCue. We want to avoid virtualizing functions on the actual user types.
// ----------------------------------------------------------------------------------------------------------------------------------------------

struct FNetSimCueWrapperBase
{
	virtual ~FNetSimCueWrapperBase() = default;
	virtual void NetSerialize(FArchive& Ar) = 0;
	virtual bool NetIdentical(const void* OtherCueData) const = 0;
	virtual void* CueData() const = 0;
	virtual ENetSimCueReplicationTarget GetReplicationTarget() const = 0;
};

template<typename TCue>
struct TNetSimCueWrapper : FNetSimCueWrapperBase
{
	TNetSimCueWrapper() = default;

	template <typename... ArgsType>
	TNetSimCueWrapper(ArgsType&&... Args)
		: Instance( MoveTempIfPossible(Forward<ArgsType>(Args))... ) { }

	void NetSerialize(FArchive& Ar) override final
	{
		TNetCueNetSerializeHelper<TCue>::CallNetSerializeOrNot(Instance, Ar);
	}

	bool NetIdentical(const void* OtherCueData) const override final
	{
		return TNetCueNetIdenticalHelper<TCue>::CallNetIdenticalOrNot(Instance, *((const TCue*)OtherCueData));
	}

	void* CueData() const override final
	{
		return (void*)&Instance;
	}

	ENetSimCueReplicationTarget GetReplicationTarget() const override final
	{
		return TNetSimCueTraits<TCue>::ReplicationTarget;
	}

	TCue Instance;
};

// ----------------------------------------------------------------------------------------------------------------------------------------------
//	Callbacks
// ----------------------------------------------------------------------------------------------------------------------------------------------

struct FNetSimCueCallbacks
{
	/** Callback to rollback any side effects of the cue. */
	DECLARE_MULTICAST_DELEGATE(FOnRollback)
	FOnRollback	OnRollback;

	// It may make sense to add an "OnConfirmed" that will let the user know a rollback will no longer be possible
};

/** System parameters for NetSimCue events */
struct FNetSimCueSystemParamemters
{
	// How much simulation time has passed since this cue was invoked. This will be 0 in authority/predict contexts, but when invoked via replication this will tell you how long ago it happened, relative to local simulation time.
	const int32& TimeSinceInvocation;

	// Callback structure if applicable. This will be null on non-rewindable cues as well as execution contexts where rollbacks wont happen (e.g, authority).
	FNetSimCueCallbacks* Callbacks;
};

// ----------------------------------------------------------------------------------------------------------------------------------------------
// GlobalCueTypeTable: Cue types register with this to get a Type ID assigned (TCue::ID). That ID is used in net serialization to talk about types.
//	-Assigns deterministic ID to TCue::ID for all cue types
//	-Provides allocation function to instantiate TCue from a NetSerialize ID
// ----------------------------------------------------------------------------------------------------------------------------------------------
class FGlobalCueTypeTable
{
public:

	// --------------------------------------------------------------------------------------
	//	Type Registration functions: called at startup, module load/unload. Not during runtime.
	// --------------------------------------------------------------------------------------

	template<typename TCue>
	FORCENOINLINE static void RegisterCueType(const FString& TypeName)
	{
		FRegisteredCueTypeInfo& RegisteredTypes = GetRegistedTypeInfo();
		UE_LOGF(LogNetworkPredictionCues, Verbose, "RegisterCueType %ls @ 0x%llX. (NumPending: %d) ", *TypeName, &TCue::ID, RegisteredTypes.CueTypes.Num());

		RegisteredTypes.bDirty = true;

		// Create a TypeInfo for this TCue and store it in the list
		FTypeInfo TypeInfo =
		{
			&TCue::ID,
			[](){ return new TNetSimCueWrapper<TCue>();},
			TypeName
		};
		RegisteredTypes.CueTypes.Emplace(TypeInfo);

		static_assert(THasMemberFunction_NetSerialize<TCue>::Value || !TNetSimCueTypeRequirements<TCue>::RequiresNetSerialize, "TCue must implement void NetSerialize(FArchive&)");
		static_assert(THasNetIdenticalHelper<TCue>::Value || !TNetSimCueTypeRequirements<TCue>::RequiresNetIdentical, "TCue must implement bool NetIdentical(const TCue&) const");
	}

	template<typename TCue>
	static void UnregisterCueType()
	{
		if (IsEngineExitRequested())
		{
			return;
		}

		UE_LOGF(LogNetworkPredictionCues, Verbose, "UnregisterCueType 0x%llX.", &TCue::ID);

		FRegisteredCueTypeInfo& RegisteredTypes = GetRegistedTypeInfo();

		for (int32 i=RegisteredTypes.CueTypes.Num()-1; i >= 0; --i)
		{
			FTypeInfo& TypeInfo = RegisteredTypes.CueTypes[i];
			if (TypeInfo.IDPtr == &TCue::ID)
			{
				RegisteredTypes.CueTypes.RemoveAtSwap(i, EAllowShrinking::No);
				RegisteredTypes.bDirty = true;
				break;
			}
		}

		// Thet TypeInfoMap has to be immediately invalidated, since it contains a TFunction that was allocated in memory that is (probably) about to go away
		Singleton.TypeInfoMap.Reset();
		npEnsureMsgf(RegisteredTypes.bDirty, TEXT("Unmatch TCue type"));
	}

	static bool IsRegisterationTypeInfoDirty() { return GetRegistedTypeInfo().bDirty; }

	static FDelegateHandle RegisterDispatchTableCallback(const TFunction<void()>& RegisterFunc)
	{
		FRegisteredCueTypeInfo& RegisteredTypes = GetRegistedTypeInfo();
		RegisteredTypes.bDirty = true;

		return RegisteredTypes.InitDispatchTablesDelegate.AddLambda( RegisterFunc );
	}

	static void UnregisterDispatchTableCallback(FDelegateHandle Handle)
	{
		if (IsEngineExitRequested())
		{
			return;
		}

		FRegisteredCueTypeInfo& RegisteredTypes = GetRegistedTypeInfo();
		RegisteredTypes.bDirty = true;

		RegisteredTypes.InitDispatchTablesDelegate.Remove(Handle);
	}

	void FinalizeCueTypes()
	{
		FRegisteredCueTypeInfo& RegisteredTypes = GetRegistedTypeInfo();
		if (RegisteredTypes.bDirty)
		{
			UE_LOGF(LogNetworkPredictionCues, Verbose, "FinalizeTypes: 0x%llX. %d Pending Types", &RegisteredTypes, RegisteredTypes.CueTypes.Num());
			RegisteredTypes.bDirty = false;

			RegisteredTypes.CueTypes.Sort([](const FTypeInfo& LHS, const FTypeInfo& RHS) { return LHS.TypeName < RHS.TypeName; });
			int32 ID=0;

			for (FTypeInfo& TypeInfo: RegisteredTypes.CueTypes)
			{
				check(TypeInfo.IDPtr != nullptr);
				*TypeInfo.IDPtr = ++ID;
				TypeInfoMap.Add(*TypeInfo.IDPtr) = TypeInfo;

				UE_LOGF(LogNetworkPredictionCues, Verbose, "    Cue %ls assigned ID: %d (0x%llX)", *TypeInfo.TypeName, *TypeInfo.IDPtr, TypeInfo.IDPtr);
			}

			RegisteredTypes.InitDispatchTablesDelegate.Broadcast();
		}
	}

	// --------------------------------------------------------------------------------------
	//	Runtime functions
	// --------------------------------------------------------------------------------------

	static FGlobalCueTypeTable& Get()
	{
		return Singleton;
	}

	FNetSimCueWrapperBase* Allocate(FNetSimCueTypeId ID)
	{
		return TypeInfoMap.FindChecked(ID).AllocateFunc();
	}

	FString GetTypeName(FNetSimCueTypeId ID) const
	{
		return TypeInfoMap.FindChecked(ID).TypeName;
	}

private:

	struct FTypeInfo
	{
		FNetSimCueTypeId* IDPtr = nullptr;
		TFunction<FNetSimCueWrapperBase*()> AllocateFunc;
		FString TypeName;
	};

	struct FRegisteredCueTypeInfo
	{
		TArray<FTypeInfo> CueTypes;
		FSimpleMulticastDelegate InitDispatchTablesDelegate;
		bool bDirty = false;
	};

	// The registered type info lives in its own inlined static function to avoid static variable initialization order conflicts
	NETWORKPREDICTION_API static FRegisteredCueTypeInfo& GetRegistedTypeInfo();

	TMap<FNetSimCueTypeId, FTypeInfo> TypeInfoMap;

	NETWORKPREDICTION_API static FGlobalCueTypeTable Singleton;
};

// ----------------------------------------------------------------------------------------------------------------------------------------------
// SavedCue: a recorded Invocation of a NetSimCue
// ----------------------------------------------------------------------------------------------------------------------------------------------

struct FSavedCue
{
	FSavedCue(bool bInNetConfirmed) : bNetConfirmed(bInNetConfirmed) {}

	FSavedCue(const FSavedCue&) = delete;
	FSavedCue& operator=(const FSavedCue&) = delete;

	FSavedCue(FSavedCue&&) = default;
	FSavedCue& operator=(FSavedCue&&) = default;

	FSavedCue(const FNetSimCueTypeId& InId, const int32& InFrame, const int32& InTime, const bool& bInAllowRollback, const bool& bInNetConfirmed, const bool& bInResimulates, FNetSimCueWrapperBase* Cue)
		: ID(InId), Frame(InFrame), Time(InTime), CueInstance(Cue), bAllowRollback(bInAllowRollback), bNetConfirmed(bInNetConfirmed), bResimulates(bInResimulates), ReplicationTarget(Cue->GetReplicationTarget())
	{

	}

	void NetSerialize(FArchive& Ar, const bool bSerializeFrameNumber)
	{
		npCheckSlow(FGlobalCueTypeTable::IsRegisterationTypeInfoDirty() == false);

		if (Ar.IsSaving())
		{
			check(CueInstance.IsValid());
			Ar << ID;
			CueInstance->NetSerialize(Ar);

			if (bSerializeFrameNumber)
			{
				Ar << Frame; // Fixme: use WriteCompressedFrame
			}
			else
			{
				Ar << Time;
			}
		}
		else
		{
			Ar << ID;
			CueInstance.Reset(FGlobalCueTypeTable::Get().Allocate(ID));
			CueInstance->NetSerialize(Ar);

			if (bSerializeFrameNumber)
			{
				Ar << Frame; // Fixme: use WriteCompressedFrame
			}
			else
			{
				Ar << Time;
			}
			ReplicationTarget = CueInstance->GetReplicationTarget();
		}
	}

	// Test NetUniqueness against another saved cue
	bool NetIdentical(FSavedCue& OtherCue) const
	{
		return (ID == OtherCue.ID) && CueInstance->NetIdentical(OtherCue.CueInstance->CueData());
	}

	// Test NetUniqueness against an actual cue instance
	template<typename TCue>
	bool NetIdentical(TCue& OtherCue) const
	{
		return (ID == TCue::ID) && CueInstance->NetIdentical(&OtherCue);
	}


	FString GetDebugName() const
	{
		return FString::Printf(TEXT("[%s 0x%" INT64_FMT "] @ (Frame %d/%dms)"), *FGlobalCueTypeTable::Get().GetTypeName(ID), (int64)this, Frame, Time);
	}

	FNetSimCueTypeId ID = 0;
	int32 Frame = INDEX_NONE;
	int32 Time = 0;
	TUniquePtr<FNetSimCueWrapperBase> CueInstance;
	FNetSimCueCallbacks Callbacks;

	bool bDispatched = false;		// Cue has been dispatched to the local handler. Never dispatch twice.
	bool bAllowRollback = false;	// Cue supports rolling back. ie., we should pass the user valid FNetSimCueCallbacks rollback callback (note this is "this saved cue" specifically, not "this cue type". E.g, on authority, bAllowRollback is always false)
	bool bNetConfirmed = false;		// This cue has been net confirmed, meaning we received it directly via replication or we received a replicated cue that matched this one that was locally predicted.

	bool bResimulates = false;				 // Whether this cue supports invocation during resimulation. Needed to set bResimulatePendingRollback
	bool bPendingResimulateRollback = false; // Rollback is pending due to resimulation (unless CUe is matched with an Invocation during the resimulate)

	ENetSimCueReplicationTarget ReplicationTarget;
};


// ----------------------------------------------------------------------------------------------------------------------------------------------
// Per-Receiver dispatch table. This is how we go from a serialized ID to a function call
// ----------------------------------------------------------------------------------------------------------------------------------------------

template<typename TCueHandler>
class TCueDispatchTable
{
public:
	static TCueDispatchTable<TCueHandler>& Get()
	{
		return Singleton;
	}

	// All types that the receiver can handle must be registered here. This is where we create the TFunction to call ::HandleCue
	template<typename TCue>
	void RegisterType()
	{
		check(TCue::ID != 0);	// FGlobalCueTypeTable should have assigned an ID to this cue by now

		FCueTypeInfo& CueTypeInfo = CueTypeInfoMap.Add(TCue::ID);

		// The actual Dispatch func that gets invoked
		CueTypeInfo.Dispatch = [](FNetSimCueWrapperBase* Cue, TCueHandler& Handler, const FNetSimCueSystemParamemters& SystemParameters)
		{
			// If you are finding compile errors here, you may be missing a ::HandleCue implementation for a specific cue type that your handler has registered with
			Handler.HandleCue( *static_cast<const TCue*>(Cue->CueData()), SystemParameters );
		};
	}

	void Reset()
	{
		CueTypeInfoMap.Reset();
	}

	void Dispatch(FSavedCue& SavedCue, TCueHandler& Handler, const int32& TimeSinceInvocation)
	{
		npCheckSlow(FGlobalCueTypeTable::IsRegisterationTypeInfoDirty() == false);

		if (FCueTypeInfo* TypeInfo = CueTypeInfoMap.Find(SavedCue.ID))
		{
			check(TypeInfo->Dispatch);
			TypeInfo->Dispatch(SavedCue.CueInstance.Get(), Handler, {TimeSinceInvocation, SavedCue.bAllowRollback ? &SavedCue.Callbacks : nullptr });
		}
		else
		{
			// Trying to dispatch a Cue to a handler who doesn't have an implementation of HandleCue for this type (or forgot to register)
			UE_LOGF(LogNetworkPredictionCues, Warning, "Could not Find NetCue %ls (ID: %d) in dispatch table (%d entries).", *FGlobalCueTypeTable::Get().GetTypeName(SavedCue.ID), SavedCue.ID, CueTypeInfoMap.Num());
			for (auto MapIt : CueTypeInfoMap)
			{
				UE_LOGF(LogNetworkPredictionCues, Warning, "  Have: %d", MapIt.Key);
			}
		}
	}

private:

	static TCueDispatchTable<TCueHandler> Singleton;

	struct FCueTypeInfo
	{
		TFunction<void(FNetSimCueWrapperBase* Cue, TCueHandler& Handler, const FNetSimCueSystemParamemters& SystemParameters)> Dispatch;
	};

	TMap<FNetSimCueTypeId, FCueTypeInfo> CueTypeInfoMap;
};

template<typename TCueHandler>
TCueDispatchTable<TCueHandler> TCueDispatchTable<TCueHandler>::Singleton;

// ------------------------------------------------------------------------------------------------------------------------------------------------------------
//	CueDispatcher
//	-Entry point for invoking cues during a SimulationTick
//	-Holds recorded cue state for replication
// ------------------------------------------------------------------------------------------------------------------------------------------------------------

// Traits for TNetSimCueDispatcher
struct FCueDispatcherTraitsBase
{
	// Window for replicating a NetSimCue. That is, after a cue is invoked, it has ReplicationWindow time before it will be pruned.
	// If a client does not get a net update for the sim in this window, they will miss the event.
	static constexpr int32 ReplicationWindowMS() { return 200; } // for time based cases
	static constexpr int32 ReplicationWindowFrames() { return 16; } // for frame based cases (FIXME: time makes more sense but we don't have good way to map at this level)
};
template<typename T> struct TCueDispatcherTraits : public FCueDispatcherTraitsBase { };

// Non-templated, "networking model independent" base: this is what the pure simulation code gets to invoke cues.
struct FNetSimCueDispatcher
{
	virtual ~FNetSimCueDispatcher() = default;

	// Invoke - this is how to invoke a cue from simulation code. This will construct the CueType T emplace in the saved cue record.
	//
	// Best way to call:
	//	Invoke<FMyCue>(a, b, c); // a, b, c are constructor parameters
	//
	// This works too, but will cause a move (if possible) or copy
	//	FMyCue MyCue(a,b,c);
	//	Invoke<FMyCue>(MyCue);

	template<typename T, typename... ArgsType>
	void Invoke(ArgsType&&... Args)
	{
		npCheckSlow(FGlobalCueTypeTable::IsRegisterationTypeInfoDirty() == false);

		if (EnsureValidContext())
		{
			if (EnumHasAnyFlags(TNetSimCueTraits<T>::SimTickMask(), Context.TickContext))
			{
				constexpr bool bSupportsResimulate = TNetSimCueTraits<T>::Resimulate;

				bool bAllowRollback = false;	// Whether this cue should be dispatched with rollback callbacks.
				bool bTransient = false;		// Whether we go in the transient list. Transient cues are dumped after dispatching (not saved over multiple frames for uniqueness comparisons during Invoke or NetSerialize)
				bool bNetConfirmed = false;		// Is this already confirmed? (we should not look to undo it if we don't get it confirmed from the server)
				if (Context.TickContext == ESimulationTickContext::Authority)
				{
					// Authority: never rolls back, is already confirmed, and can treat cue as transient if it doesn't have to replicate it
					bAllowRollback = false;
					bTransient = TNetSimCueTraits<T>::ReplicationTarget == ENetSimCueReplicationTarget::None;
					bNetConfirmed = true;
				}
				else
				{
					// Everyone else that is running the simulation (since this in ::Invoke which is called from within the simulation)
					// Rollback if it will replicate to us or if we plan to invoke this cue during resimulates. Transient and confirmed follow directly from this in the non authority case.
					bAllowRollback = (TNetSimCueTraits<T>::ReplicationTarget == ENetSimCueReplicationTarget::All) || bSupportsResimulate;
					bTransient = !bAllowRollback;
					bNetConfirmed = !bAllowRollback;
				}

				npEnsure(!(bTransient && bAllowRollback)); // this combination cannot happen: we can't be transient and support rollback (but we can be transient without supporting rollback)
				npEnsure(!(bNetConfirmed && bAllowRollback)); // a confirmed cue shouldn't be rolled back.

				// In resimulate case, we have to see if we already predicted it
				if (Context.TickContext == ESimulationTickContext::Resimulate)
				{
					npEnsure(RollbackFrame >= 0 && RollbackFrame <= Context.Frame);

					// Since we haven't constructed the cue yet, we can't test for uniqueness!
					// So, create one on the stack. If we let it through we can move it to the appropriate buffer
					// (not as nice as the non resimulate path, but better than allocating a new FSavedCue+TNetSimCueWrapper and then removing them)
					T NewCue(MoveTempIfPossible(Forward<ArgsType>(Args))...);

					for (FSavedCue& ExistingCue : SavedCues)
					{
						npEnsureSlow(ExistingCue.Frame != INDEX_NONE);
						if (RollbackFrame <= ExistingCue.Frame && ExistingCue.NetIdentical(NewCue))
						{
							// We've matched with an already predicted cue, so suppress this invocation and don't undo the predicted cue
							ExistingCue.bPendingResimulateRollback = false;
							UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Resimulated Cue %ls matched existing cue. Suppressing Invocation. Time: %d", *GetDebugName(), *FGlobalCueTypeTable::Get().GetTypeName(T::ID), Context.CurrentSimTime);
							return;
						}
					}


					auto& SavedCue = GetBuffer(bTransient).Emplace_GetRef(T::ID, Context.Frame, Context.CurrentSimTime, bAllowRollback, bNetConfirmed, bSupportsResimulate, new TNetSimCueWrapper<T>(MoveTempIfPossible(NewCue)));
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Invoking Cue %ls during resimulate. Context: %d. Transient: %d. bAllowRollback: %d. bNetConfirmed: %d. Time: %d", *GetDebugName(), *SavedCue.GetDebugName(), int(Context.TickContext), bTransient, bAllowRollback, bNetConfirmed, Context.CurrentSimTime);
				}
				else
				{
					// Not resimulate case is simple: construct the new cue emplace in the appropriate list
					auto& SavedCue = GetBuffer(bTransient).Emplace_GetRef(T::ID, Context.Frame, Context.CurrentSimTime, bAllowRollback, bNetConfirmed, bSupportsResimulate, new TNetSimCueWrapper<T>(Forward<ArgsType>(Args)...));
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Invoking Cue %ls. Context: %d. Transient: %d. bAllowRollback: %d. bNetConfirmed: %d. Time: %d", *GetDebugName(), *SavedCue.GetDebugName(), int(Context.TickContext), bTransient, bAllowRollback, bNetConfirmed, Context.CurrentSimTime);
				}
			}
			else
			{
				UE_LOGF(LogNetworkPredictionCues, Log, "%ls .Suppressing Cue Invocation %ls. Mask: %d. TickContext: %d. Time: %d", *GetDebugName(), *FGlobalCueTypeTable::Get().GetTypeName(T::ID), int(TNetSimCueTraits<T>::SimTickMask()), (int32)Context.TickContext, Context.CurrentSimTime);
			}
		}
	}

	virtual FString GetDebugName() const = 0;

protected:

	bool EnsureValidContext()
	{
		return npEnsure(Context.CurrentSimTime > 0 && Context.TickContext != ESimulationTickContext::None);
	}

	// Sim Context: the Sim has to tell the dispatcher what its doing so that it can decide if it should suppress Invocations or not
	struct FContext
	{
		int32 Frame = INDEX_NONE;
		int32 CurrentSimTime = 0;
		ESimulationTickContext TickContext;
	};

	TArray<FSavedCue> SavedCues;		// Cues that must be saved for some period of time, either for replication or for uniqueness testing
	TArray<FSavedCue> TransientCues;	// Cues that are dispatched on this frame and then forgotten about
	TArray<FSavedCue>& GetBuffer(const bool& bTransient) { return bTransient ? TransientCues : SavedCues; }

	FContext Context;
	int32 RollbackFrame = INDEX_NONE;		// Frame# of last rollback, reset after dispatching

public:

	// Push/pop simulation context.
	void PushContext(const FContext& InContext) { Context = InContext; }
	void PopContext() { Context = FContext(); }
	const FContext& GetContext() const { return Context; }

};

// Templated cue dispatcher that can be specialized per networking model definition. This is what the system actually uses internally, but is not exposed to user code.
template<typename ModelDef=void>
struct TNetSimCueDispatcher : public FNetSimCueDispatcher
{
	using DriverType = typename ModelDef::Driver;

	// Serializes all saved cues
	//	bSerializeFrameNumber - whether to serialize Frame# or Time to this target. Frame is more accurate but some cases require time based replication.
	void NetSendSavedCues(FArchive& Ar, ENetSimCueReplicationTarget ReplicationMask, bool bSerializeFrameNumber)
	{
		npCheckSlow(Ar.IsSaving());

		// FIXME: requires two passes to count how many elements are valid for this replication mask.
		// We could count this as saved cues are added or possibly modify the bitstream after writing the elements (tricky and would require casting to FNetBitWriter which feels real bad)
		FNetSimCueTypeId NumCues = 0;
		for (FSavedCue& SavedCue : SavedCues)
		{
			if (EnumHasAnyFlags(SavedCue.ReplicationTarget, ReplicationMask))
			{
				NumCues++;
			}
		}

		Ar << NumCues;

		for (FSavedCue& SavedCue : SavedCues)
		{
			if (EnumHasAnyFlags(SavedCue.ReplicationTarget, ReplicationMask))
			{
				SavedCue.NetSerialize(Ar, bSerializeFrameNumber);
			}
		}

	}

	void NetRecvSavedCues(FArchive& Ar, const bool bSerializeFrameNumber, const int32 InLastRecvFrame, const int32 InLastRecvTime)
	{
		npCheckSlow(Ar.IsLoading());

		FNetSimCueTypeId NumCues;
		Ar << NumCues;

		// This is quite inefficient right now.
		//	-We are replicating cues in the last X seconds/frames (ReplicationWindow) redundantly
		//	-Client has to deserialize them (+ heap allocation) and check for uniqueness (have they already processed)
		//	-If already processed (quite common), they are thrown out.
		//	-Would be better if we maybe serialized "net hash" and could skip ahead in the bunch if already processed

		int32 StartingNum = SavedCues.Num();

		for (int32 CueIdx=0; CueIdx < NumCues; ++CueIdx)
		{
			FSavedCue SerializedCue(true);
			SerializedCue.NetSerialize(Ar, bSerializeFrameNumber);

			// Decide if we should accept the cue:
			// ReplicationTarget: Cues can be set to only replicate to interpolators
			if (EnumHasAnyFlags(SerializedCue.ReplicationTarget, RecvReplicationMask) == false)
			{
				UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Discarding replicated NetSimCue %ls that is not intended for us. CueMask: %d. Our Mask: %d", *GetDebugName(), *SerializedCue.GetDebugName(), int(SerializedCue.ReplicationTarget), int(RecvReplicationMask));
				continue;
			}

			// Due to redundant sending, we may get frames older than what we've already processed. Early out rather than searching for them in saved cues.
			// This also ensures that if we get a very stale cue that we have pruned locally, we won't incorrectly invoke it again.
			if (SerializedCue.Frame != INDEX_NONE)
			{
				if (SerializedCue.Frame <= LastRecvFrame)
				{
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Discarding replicated NetSimCue %ls because it is older (%d) than confirmed frame (%d). CueMask: %d. Our Mask: %d", *GetDebugName(), *SerializedCue.GetDebugName(), SerializedCue.Frame, LastRecvFrame, int(SerializedCue.ReplicationTarget), int(RecvReplicationMask));
					continue;
				}
			}
			else if (SerializedCue.Time <= LastRecvMS)
			{
				UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Discarding replicated NetSimCue %ls because it is older (%d) than confirmed TimeMS (%d). CueMask: %d. Our Mask: %d", *GetDebugName(), *SerializedCue.GetDebugName(), SerializedCue.Frame, LastRecvFrame, int(SerializedCue.ReplicationTarget), int(RecvReplicationMask));
				continue;
			}

			// Uniqueness: have we already received/predicted it?
			// Note: we are basically ignoring invocation time when matching right now. This could potentially be a trait of the cue if needed.
			// This could create issues if a cue is invoked several times in quick succession, but that can be worked around with arbitrary counter parameters on the cue itself (to force NetUniqueness)
			bool bUniqueCue = true;
			for (int32 ExistingIdx=0; ExistingIdx < StartingNum; ++ExistingIdx)
			{
				FSavedCue& ExistingCue = SavedCues[ExistingIdx];
				if (SerializedCue.NetIdentical(ExistingCue))
				{
					// These cues are not unique ("close enough") so we are skipping receiving this one
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Discarding replicated NetSimCue %ls because we've already processed it. (Matched %ls)", *GetDebugName(), *SerializedCue.GetDebugName(), *ExistingCue.GetDebugName());
					bUniqueCue = false;
					ExistingCue.bNetConfirmed = true;
					break;
				}
			}

			if (bUniqueCue)
			{
				auto& SavedCue = SavedCues.Emplace_GetRef(MoveTemp(SerializedCue));
				UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Received !NetIdentical Cue: %ls. (Num replicated cue sent this bunch: %d. LastRecvFrame: %d. Our Mask: %d).", *GetDebugName(), *SerializedCue.GetDebugName(), NumCues, LastRecvFrame, int(RecvReplicationMask));
			}
		}

		LastRecvFrame = InLastRecvFrame;
		LastRecvMS = InLastRecvTime;
	}

	// Dispatches and prunes saved/transient cues
	template<typename T>
	void DispatchCueRecord(T& Handler, const int32 SimFrame, const int32 CurrentSimTime, const int32 FixedTimeStepMS)
	{
		auto CalcPrune = [](int32 Head, int32 Confirm, int32 Window) { return Confirm > 0 ? FMath::Min(Confirm, Head - Window) : Head - Window; };

		const int32 SavedCuePruneTimeMS = CalcPrune(CurrentSimTime, LastRecvMS, TCueDispatcherTraits<ModelDef>::ReplicationWindowMS());
		const int32 SavedCuePruneFrame = CalcPrune(SimFrame, LastRecvFrame, TCueDispatcherTraits<ModelDef>::ReplicationWindowFrames());

		// Prune cues prior to this idx
		int32 SavedCuePruneIdx = INDEX_NONE;

		// ------------------------------------------------------------------------
		// Rollback events if necessary
		//	Fixme - this code was written for clarity, it could be sped up considerably by taking advantage of sorting by time, or keeping acceleration lists for this type of pruning
		// ------------------------------------------------------------------------

		if (LastRecvFrame != INDEX_NONE)
		{
			// Look for cues that should have been matched by now, but were not
			for (auto It = SavedCues.CreateIterator(); It; ++It)
			{
				FSavedCue& SavedCue = *It;
				npEnsureSlow(SavedCue.Frame != INDEX_NONE);
				if (!SavedCue.bNetConfirmed && SavedCue.Frame <= LastRecvFrame)
				{
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Calling OnRollback for SavedCue NetSimCue %ls. Cue has not been matched but it <= LastRecvFrame %d.", *GetDebugName(), *SavedCue.GetDebugName(), LastRecvFrame);
					SavedCue.Callbacks.OnRollback.Broadcast();
					SavedCues.RemoveAt(It.GetIndex(), EAllowShrinking::No);
				}
			}
		}

		if (RollbackFrame >= 0)
		{
			for (auto It = SavedCues.CreateIterator(); It; ++It)
			{
				FSavedCue& SavedCue = *It;
				if (SavedCue.bPendingResimulateRollback)
				{
					// Unmatched cue whose time has passed, time to rollback
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Calling OnRollback for SavedCue NetSimCue %ls. Cue was not matched during a resimulate.", *GetDebugName(), *SavedCue.GetDebugName());
					SavedCue.Callbacks.OnRollback.Broadcast();
					SavedCues.RemoveAt(It.GetIndex(), EAllowShrinking::No);
				}
			}

			RollbackFrame = INDEX_NONE;
		}

		// ------------------------------------------------------------------------
		// Dispatch (call ::HandleCue)
		// ------------------------------------------------------------------------

		for (int32 SavedCueIdx = 0; SavedCueIdx < SavedCues.Num(); ++ SavedCueIdx)
		{
			FSavedCue& SavedCue = SavedCues[SavedCueIdx];

			// See if Cue is ready for prune
			const bool bCueIsReadyForPrune = (SavedCue.Frame == INDEX_NONE) ?  (SavedCue.Time <= SavedCuePruneTimeMS) : (SavedCue.Frame <= SavedCuePruneFrame);
			if (bCueIsReadyForPrune)
			{
				SavedCuePruneIdx = SavedCueIdx;
			}

			if (!SavedCue.bDispatched)
			{
				bool bDispatchedCue = false;
				int32 TimeSinceInvocationMS = 0;

				if (SavedCue.Frame != INDEX_NONE)
				{
					// Frame based comparison (prefer if available)
					if (SavedCue.Frame <= SimFrame)
					{
						const int32 FramesSinceInvoke = SimFrame - SavedCue.Frame;
						TimeSinceInvocationMS = FramesSinceInvoke * FixedTimeStepMS;
						bDispatchedCue = true;
					}
				}
				else
				{
					// Time based comparison
					if (SavedCue.Time <= CurrentSimTime)
					{
						TimeSinceInvocationMS = CurrentSimTime - SavedCue.Time;
						bDispatchedCue = true;
					}
				}

				if (bDispatchedCue)
				{
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls Dispatching NetSimCue %ls {Frame: %d, Time: %d}", *GetDebugName(), *SavedCue.GetDebugName(), SavedCue.Frame, SavedCue.Time);
					SavedCue.bDispatched = true;
					TCueDispatchTable<T>::Get().Dispatch(SavedCue, Handler, TimeSinceInvocationMS);
				}
				else
				{
					UE_LOGF(LogNetworkPredictionCues, Log, "%ls Withholding Cue %ls. %d/%d > %d/%d", *GetDebugName(), *SavedCue.GetDebugName(), SavedCue.Frame, SavedCue.Time, SimFrame, CurrentSimTime);
				}
			}
		}

		for (FSavedCue& TransientCue : TransientCues)
		{
			UE_LOGF(LogNetworkPredictionCues, Log, "Dispatching transient NetSimCue %ls.", *TransientCue.GetDebugName());
			TCueDispatchTable<T>::Get().Dispatch(TransientCue, Handler, CurrentSimTime);
		}
		TransientCues.Reset();

		// ------------------------------------------------------------------------
		// Prune
		// ------------------------------------------------------------------------

		// Remove Cues we know longer need to keep around
		if (SavedCuePruneIdx >= 0)
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			for (int32 i=0; i <= SavedCuePruneIdx; ++i)
			{
				UE_CLOGF(!SavedCues[i].bDispatched, LogNetworkPredictionCues, Warning, "Non-Dispatched Cue is about to be pruned! %ls. SavedCuePruneTimeMS: %d. SavedCuePruneFrame: %d", *SavedCues[i].GetDebugName(), SavedCuePruneTimeMS, SavedCuePruneFrame);
				UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Pruning Cue %ls. Prune: (%d/%d). Invoked: (%d/%d). Current Time: %d.", *GetDebugName(), *SavedCues[i].GetDebugName(), SavedCuePruneFrame, SavedCuePruneTimeMS,  SavedCues[i].Frame, SavedCues[i].Time, CurrentSimTime);
			}
#endif

			SavedCues.RemoveAt(0, SavedCuePruneIdx+1, EAllowShrinking::No);
		}
	}

	// Tell dispatcher that we've rolled back to a new simulation time (resimulate steps to follow, most likely)
	void NotifyRollback(int32 InRollbackFrame)
	{
		// Just cache off the notify. We want to invoke the callbacks in DispatchCueRecord, not right now (in the middle of simulation tick/reconcile)
		if (RollbackFrame <= 0)
		{
			RollbackFrame = InRollbackFrame;
		}
		else
		{
			// Two rollbacks could happen in between DispatchCueRecord calls. That is ok as long as the subsequent rollbacks are further ahead in simulation time
			npEnsure(RollbackFrame <= InRollbackFrame);
		}

		// Mark all cues that support invocation during resimulation as pending rollback (unless they match in an Invoke)
		for (FSavedCue& SavedCue : SavedCues)
		{
			npEnsureSlow(SavedCue.Frame != INDEX_NONE);
			if (SavedCue.bResimulates && SavedCue.Frame >= InRollbackFrame)
			{
				SavedCue.bPendingResimulateRollback = true;
				UE_LOGF(LogNetworkPredictionCues, Log, "%ls. Marking %ls bPendingResimulateRollback. (RollbackFrame: %d/%d)", *GetDebugName(), *SavedCue.GetDebugName(), RollbackFrame, InRollbackFrame);
			}
		}
	}

	// Set which cues we should accept locally (if they get sent to us)
	void SetReceiveReplicationTarget(ENetSimCueReplicationTarget InReplicationMask)
	{
		RecvReplicationMask = InReplicationMask;
	}

	DriverType* Driver = nullptr;
	FString GetDebugName() const final override
	{
		FString Str;
		if (Driver)
		{
			TStringBuilder<128> Builder;
			FNetworkPredictionDriver<ModelDef>::GetDebugString(Driver, Builder);

			Str = FString(Builder.ToString());
		}

		return Str;
	}

private:

	ENetSimCueReplicationTarget RecvReplicationMask;

	int32 LastRecvFrame = INDEX_NONE;
	int32 LastRecvMS = 0;
};

// ------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------
//	Registration Helpers and Macros
// ------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------------

// Body of NetSimCue header. Gives type a static ID to be identified with
#define NETSIMCUE_BODY() static FNetSimCueTypeId ID

// ------------------------------------------------------------------------------------------------
// Register a cue type with the global table.
// ------------------------------------------------------------------------------------------------
template<typename TCue>
struct TNetSimCueTypeAutoRegisterHelper
{
	TNetSimCueTypeAutoRegisterHelper(const FString& Name)
	{
		FGlobalCueTypeTable::RegisterCueType<TCue>(Name);
	}

	~TNetSimCueTypeAutoRegisterHelper()
	{
		FGlobalCueTypeTable::UnregisterCueType<TCue>();
	}
};

// Note this also defines the internal static ID for the cue type
#define NETSIMCUE_REGISTER(X, STR) FNetSimCueTypeId X::ID=0; TNetSimCueTypeAutoRegisterHelper<X> NetSimCueAr_##X(STR);

// ------------------------------------------------------------------------------------------------
// Register a handler's cue types via a static "RegisterNetSimCueTypes" on the handler itself
// ------------------------------------------------------------------------------------------------
template<typename THandler>
struct TNetSimCueHandlerAutoRegisterHelper
{
	TNetSimCueHandlerAutoRegisterHelper()
	{
		Handle = FGlobalCueTypeTable::RegisterDispatchTableCallback([]()
		{
			TCueDispatchTable<THandler>::Get().Reset();
			THandler::RegisterNetSimCueTypes( TCueDispatchTable<THandler>::Get() );
		});
	}

	~TNetSimCueHandlerAutoRegisterHelper()
	{
		FGlobalCueTypeTable::UnregisterDispatchTableCallback(Handle);
	}

private:
	FDelegateHandle Handle;
};

#define NETSIMCUEHANDLER_REGISTER(X) TNetSimCueHandlerAutoRegisterHelper<X> NetSimCueHandlerAr_##X;

// ------------------------------------------------------------------------------------------------
// Register a handler's cue types via an intermediate "set" class with a static "RegisterNetSimCueTypes" function
// ------------------------------------------------------------------------------------------------
template<typename THandler, typename TCueSet>
struct TNetSimCueSetHandlerAutoRegisterHelper
{
	TNetSimCueSetHandlerAutoRegisterHelper()
	{
		Handle = FGlobalCueTypeTable::RegisterDispatchTableCallback([]()
		{
			TCueDispatchTable<THandler>::Get().Reset();
			TCueSet::RegisterNetSimCueTypes( TCueDispatchTable<THandler>::Get() );
		});
	}

	~TNetSimCueSetHandlerAutoRegisterHelper()
	{
		FGlobalCueTypeTable::UnregisterDispatchTableCallback(Handle);
	}

private:
	FDelegateHandle Handle;
};

#define NETSIMCUESET_REGISTER(THandler, TCueSet) TNetSimCueSetHandlerAutoRegisterHelper<THandler,TCueSet> NetSimCueSetHandlerAr_##THandler_##TCueSet;
