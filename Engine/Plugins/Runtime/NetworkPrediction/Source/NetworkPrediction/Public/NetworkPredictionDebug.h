// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Math/Box.h"

class FNetworkGUID;

// non ModelDef specific debug helpers
namespace NetworkPredictionDebug
{
	NETWORKPREDICTION_API void DrawDebugOutline(FTransform Transform, FBox BoundingBox, FColor Color, float Lifetime);
	NETWORKPREDICTION_API void DrawDebugText3D(const TCHAR* Str, FTransform Transform, FColor, float Lifetime);
	NETWORKPREDICTION_API UObject* FindReplicatedObjectOnPIEServer(UObject* ClientObject);
	NETWORKPREDICTION_API FNetworkGUID FindObjectNetGUID(UObject* Obj);
};
