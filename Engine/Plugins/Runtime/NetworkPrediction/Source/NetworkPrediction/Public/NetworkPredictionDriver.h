// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NetworkPredictionCues.h"
#include "NetworkPredictionConditionalState.h"
#include "NetworkPredictionSettings.h"
#include "NetworkPredictionDebug.h"
#include "NetworkPredictionStateTypes.h"
#include "NetworkPredictionStateView.h"
#include "Components/PrimitiveComponent.h"

struct FNetSerializeParams;

struct FBodyInstance;

namespace Chaos
{
	class FRewindData;
};

// The "driver" represents the UE side of the NetworkPrediction System. Typically the driver will be an Unreal Actor or ActorComponent.
// This driver class is defined in the ModelDef (your version of FNetworkPredictionModelDef). For example, AActor or AMyPawn class.
// While the Simulation class is agnostic to all of this, the driver is responsible for specifying exactly how things should work, 
// generating input and handling output from the system.
//
// Its worth pointing out that a new ModelDef with a new Driver class is only required when you want to actually change behavior at the driver
// level. For example, if you define a ModelDef {MySimulation, AMyPawn_Base} you can still use this ModelDef with both AMyPawn_Player, AMyPawn_AI.
// You would only need to create a ModelDef {MySimulation, AMyPawn_Player} if your player class wanted to customize Driver calls for example.
// (And even then: making the Driver functions virtual on AMyPawn_Base would be enough, at the cost of virtual function calls).
//
// FNetworkPredictionDriver is how calls to the driver are made. FNetworkPredictionDriverBase is a default implementation that will be used.
// You can customize any of these calls by specializing FNetworkPredictionDriver to your ModelDef type. If you do this, you should inherit from 
// FNetworkPredictionDriverBase<YourModelDef> or FNetworkPredictionDriver<ParentModelDef> if you are extending an existing ModelDef.
// (otherwise you will need to implement every required function yourself).
//
//
// The default implementations can be broken down into a few categories:
//
//	1.	Simple stuff like ::GetDebugString() - we provide generic implemenations for AActor and UActorComponent. In general you won't need 
//		to implement these yourself unless you want to include extra information in the debug string that the system prints to logs.
//
//	2.	Calls that get forwarded to the Driver itself. For example ::InitializeSimulationState(FSyncState*, FAuxState*). We can't provide
//		a generic implementation because the state type is defined by the user. We forward this to the Driver because that is ultimately
//		where the initial simulation state is seeded from. Defining InitializeSimulationState on the Driver itself is the simplest way
//		of doing this and will make the most sense to anyone looking at the Driver class.
//
//		However there may be cases when this is not an option. For example if you want to create a Simulation in this system that can be driven
//		by an AActor. You wouldn't want to modify AActor itself to implement InitializeSimulationState. In those cases, you can specialize
//		FNetworkPredictionDriver<YourModelDef>::InitializeSimulationState(AActor*, FSyncState*, FAuxState*) and implement it there.
//
//	3.	Calls that get forwarded to the underlying StateTypes. For example ::ShouldReconcile(FSyncState*, FSyncState*). The default 
//		implementation for these calls will get forwarded to member functions on the state type itself. This allows the user struct
//		to define the default behavior while still giving the Driver type the option to override.
//
//
//	High level goals: maximize non-intrusive extendability, shield users from templated boiler plate where possible.

template<typename ModelDef>
struct FNetworkPredictionDriver;

template<typename ModelDef>
struct FNetworkPredictionDriverBase
{
	using DriverType = typename ModelDef::Driver;
	using Simulation = typename ModelDef::Simulation;
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	static constexpr bool HasNpState() { return !std::is_void_v<InputType> || !std::is_void_v<SyncType> || !std::is_void_v<AuxType>; }
	static constexpr bool HasDriver() { return !std::is_void_v<DriverType>; }
	static constexpr bool HasSimulation() { return !std::is_void_v<Simulation>; }
	static constexpr bool HasInput() { return !std::is_void_v<InputType>; }

	// Defines what the ModelDef can do. This is a compile time thing only.
	static constexpr FNetworkPredictionModelDefCapabilities GetCapabilities()
	{
		FNetworkPredictionModelDefCapabilities Capabilities;

		if (HasSimulation() == false)
		{
			// We have nothing to tick, so no SimExtrapolation or ForwardPrediction
			Capabilities.FixedNetworkLODs.AP = ENetworkLOD::Interpolated;
			Capabilities.FixedNetworkLODs.SP = ENetworkLOD::Interpolated;
			Capabilities.IndependentNetworkLODs.AP = ENetworkLOD::Interpolated;
			Capabilities.IndependentNetworkLODs.SP = ENetworkLOD::Interpolated;
		}
		return Capabilities;
	}

	// Defines the default settings for a spawn instance.
	static bool GetDefaultArchetype(FNetworkPredictionInstanceArchetype& Archetype, ENetworkPredictionTickingPolicy PreferredTickingPolicy)
	{
		static constexpr FNetworkPredictionModelDefCapabilities Capabilities = FNetworkPredictionDriver<ModelDef>::GetCapabilities();

		// Use preferred ticking policy if we support it
		if (EnumHasAnyFlags(Capabilities.SupportedTickingPolicies, PreferredTickingPolicy))
		{
			Archetype.TickingMode = PreferredTickingPolicy;
		}
		else
		{
			// else use the one we support (assumes only 2 modes)
			Archetype.TickingMode = Capabilities.SupportedTickingPolicies;
		}
		
		return true;
	}

	// Defines the default config for an instance, given their archetype and Role/NetConnection
	static FNetworkPredictionInstanceConfig GetConfig(const FNetworkPredictionInstanceArchetype& Archetype, const FNetworkPredictionSettings& GlobalSettings, ENetRole Role, bool bHasNetConnection)
	{
		static constexpr FNetworkPredictionModelDefCapabilities Capabilities = FNetworkPredictionDriver<ModelDef>::GetCapabilities();
		FNetworkPredictionInstanceConfig Config;
		switch (Role)
		{
		case ROLE_Authority:
			Config.InputPolicy = bHasNetConnection ? ENetworkPredictionLocalInputPolicy::Passive : ENetworkPredictionLocalInputPolicy::PollPerSimFrame;
			break;
		case ROLE_AutonomousProxy:
			Config.InputPolicy = ENetworkPredictionLocalInputPolicy::PollPerSimFrame;
			Config.NetworkLOD = Archetype.TickingMode == ENetworkPredictionTickingPolicy::Fixed ? GetHighestNetworkLOD(Capabilities.FixedNetworkLODs.AP) : GetHighestNetworkLOD(Capabilities.IndependentNetworkLODs.AP);
			break;
		case ROLE_SimulatedProxy:
			Config.InputPolicy = ENetworkPredictionLocalInputPolicy::Passive;

			// Use preferred SP NetworkLOD if we support it
			const ENetworkLOD CapableLODs = (Archetype.TickingMode == ENetworkPredictionTickingPolicy::Fixed) ? Capabilities.FixedNetworkLODs.SP : Capabilities.IndependentNetworkLODs.SP;
			if (EnumHasAnyFlags(CapableLODs, GlobalSettings.SimulatedProxyNetworkLOD))
			{
				Config.NetworkLOD = GlobalSettings.SimulatedProxyNetworkLOD;
			}
			else
			{
				// Use highest allowed LOD otherwise
				Config.NetworkLOD = GetHighestNetworkLOD(CapableLODs);
			}
			break;
		};

		return Config;
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	Basic string/debug info
	// -----------------------------------------------------------------------------------------------------------------------------------

	static void GetDebugString(AActor* Actor, FStringBuilderBase& Builder)
	{
		Builder.Appendf(TEXT("%s %s"), ModelDef::GetName(), *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), Actor->GetLocalRole()));
	}
	
	static void GetDebugString(UActorComponent* ActorComp, FStringBuilderBase& Builder)
	{
		Builder.Appendf(TEXT("%s %s"), ModelDef::GetName(), *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), ActorComp->GetOwnerRole()));
	}

	static void GetDebugString(void* NoDriver, FStringBuilderBase& Builder)
	{
		Builder.Append(ModelDef::GetName());
	}

	static void GetDebugStringFull(AActor* Actor, FStringBuilderBase& Builder)
	{
		Builder.Appendf(TEXT("%s. Driver: %s. Role: %s."), ModelDef::GetName(), *Actor->GetPathName(), *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), Actor->GetLocalRole()));
	}

	static void GetDebugStringFull(UActorComponent* ActorComp, FStringBuilderBase& Builder)
	{
		Builder.Appendf(TEXT("%s. Driver: %s. Role: %s."), ModelDef::GetName(), *ActorComp->GetPathName(), *UEnum::GetValueAsString(TEXT("Engine.ENetRole"), ActorComp->GetOwnerRole()));
	}

	static void GetDebugStringFull(void* NoDriver, FStringBuilderBase& Builder)
	{
		Builder.Append(ModelDef::GetName());
	}

	static void GetTraceString(UActorComponent* ActorComp, FStringBuilderBase& Builder)
	{
		Builder.Appendf(TEXT("%s: %s %s"), ModelDef::GetName(), *ActorComp->GetOwner()->GetName(), *ActorComp->GetName());
	}

	static void GetTraceString(AActor* Actor, FStringBuilderBase& Builder)
	{
		Builder.Appendf(TEXT("%s: %s."), ModelDef::GetName(), *Actor->GetName());
	}
	
	static void GetTraceString(void* NoDriver, FStringBuilderBase& Builder)
	{
		Builder.Append(ModelDef::GetName());
	}

	static FTransform GetDebugWorldTransform(AActor* DriverActor)
	{
		return DriverActor->GetTransform();
	}

	static FTransform GetDebugWorldTransform(UActorComponent* DriverComponent)
	{
		return GetDebugWorldTransform(DriverComponent->GetOwner());
	}

	static FTransform GetDebugWorldTransform(void* NoDriver)
	{
		npEnsure(false);
		return FTransform::Identity;
	}

	static FBox GetDebugBoundingBox(AActor* DriverActor)
	{
		return DriverActor->CalculateComponentsBoundingBoxInLocalSpace();
	}

	static FBox GetDebugBoundingBox(UActorComponent* DriverComponent)
	{
		return GetDebugBoundingBox(DriverComponent->GetOwner());
	}

	static FBox GetDebugBoundingBox(void* NoDriver)
	{
		npEnsure(false);
		return FBox();
	}

	static void DrawDebugOutline(DriverType* Driver, FColor Color, float Lifetime=0.f)
	{
		NetworkPredictionDebug::DrawDebugOutline(GetDebugWorldTransform(Driver), GetDebugBoundingBox(Driver), Color, Lifetime);
	}

	static void DrawDebugText3D(DriverType* Driver, const TCHAR* Str, FColor Color, float Lifetime=0.f, float ZOffset=50.f)
	{
		FTransform Transform = GetDebugWorldTransform(Driver);
		Transform.AddToTranslation(FVector(0.f, 0.f, ZOffset));
		NetworkPredictionDebug::DrawDebugText3D(Str, Transform , Color, Lifetime);
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	InitializeSimulationState
	//
	//	Set the given simulation state to the current state of the driver.
	//	Called whenever the simulation is fully initialized and is ready to have its initial state set.
	//	This will be called if the instance is reconfigured as well (E.g, went from AP->SP, Interpolated->Forward Predicted, etc).
	//
	//	The default implementation will call InitializeSimulationState(FSyncState*, FAuxState*) on the Driver class itself.
	// -----------------------------------------------------------------------------------------------------------------------------------
	static void InitializeSimulationState(DriverType* Driver, FNetworkPredictionStateView* View)
	{
		npCheckSlow(View);
		InitializeSimulationState(Driver, (SyncType*)View->PendingSyncState, (AuxType*)View->PendingAuxState);
	}
	
	static void InitializeSimulationState(DriverType* Driver, SyncType* Sync, AuxType* Aux)
	{
		CallInitializeSimulationStateMemberFunc(Driver, Sync, Aux);
	}
	
	struct CInitializeSimulationStateFuncable
	{
		template <typename InDriverType, typename...>
		auto Requires(InDriverType* Driver, SyncType* Sync, AuxType* Aux) -> decltype(Driver->InitializeSimulationState(Sync, Aux));
	};

	static constexpr bool HasInitializeSimulationState = TModels_V<CInitializeSimulationStateFuncable, DriverType, SyncType, AuxType>;
	
	template<bool HasFunc=HasInitializeSimulationState>
	static typename TEnableIf<HasFunc>::Type CallInitializeSimulationStateMemberFunc(DriverType* Driver, SyncType* Sync, AuxType* Aux)
	{
		npCheckSlow(Driver);
		Driver->InitializeSimulationState(Sync, Aux);
	}

	template<bool HasFunc=HasInitializeSimulationState>
	static typename TEnableIf<!HasFunc>::Type CallInitializeSimulationStateMemberFunc(DriverType* Driver, SyncType* Sync, AuxType* Aux)
	{
		npCheckf(!HasNpState(), TEXT("No InitializeSimulationState implementation found. Implement DriverType::ProduceInput or ModelDef::ProduceInput"));
	}	

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	ProduceInput
	//
	//	Called on locally controlled simulations prior to ticking a new frame. This is to allow input to be as fresh as possible.
	//  submitting input from an Actor tick would be too late in the frame. NOTE: currently input is sampled/broadcast in PC tick and 
	//	this is still causing a frame of latency in the samples. This will be fixed in the future.
	// -----------------------------------------------------------------------------------------------------------------------------------

	static void ProduceInput(DriverType* Driver, int32 DeltaTimeMS, InputType* InputCmd)
	{
		CallProduceInputMemberFunc(Driver, DeltaTimeMS, InputCmd);
	}
	
	struct CProduceInputMemberFuncable
	{
		template <typename InDriverType, typename...>
		auto Requires(InDriverType* Driver, int32 TimeMS, InputType* InputCmd) -> decltype(Driver->ProduceInput(TimeMS, InputCmd));
	};

	static constexpr bool HasProduceInput = TModels_V<CProduceInputMemberFuncable, DriverType, int32, InputType>;

	template<bool HasFunc=HasProduceInput>
	static typename TEnableIf<HasFunc>::Type CallProduceInputMemberFunc(DriverType* Driver, int32 DeltaTimeMS, InputType* InputCmd)
	{
		npCheckSlow(Driver);
		Driver->ProduceInput(DeltaTimeMS, InputCmd);
	}

	template<bool HasFunc=HasProduceInput>
	static typename TEnableIf<!HasFunc>::Type CallProduceInputMemberFunc(DriverType* Driver, int32 DeltaTimeMS, InputType* InputCmd)
	{
		npCheckf(false, TEXT("No ProduceInput implementation found. Implement DriverType::ProduceInput or ModelDef::ProduceInput"));
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	FinalizeFrame
	//
	//	Called every engine frame to push the final result of the NetworkPrediction system to the driver
	// -----------------------------------------------------------------------------------------------------------------------------------
	static void FinalizeFrame(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		CallFinalizeFrameMemberFunc(Driver, SyncState, AuxState);
	}
	
	struct CFinalizeFrameMemberFuncable
	{
		template <typename InDriverType, typename...>
		auto Requires(InDriverType* Driver, const SyncType* S, const AuxType* A) -> decltype(Driver->FinalizeFrame(S, A));
	};

	static constexpr bool HasFinalizeFrame = TModels_V<CFinalizeFrameMemberFuncable, DriverType, SyncType, AuxType>;

	template<bool HasFunc=HasFinalizeFrame>
	static typename TEnableIf<HasFunc>::Type CallFinalizeFrameMemberFunc(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		npCheckSlow(Driver);
		Driver->FinalizeFrame(SyncState, AuxState);
	}

	template<bool HasFunc=HasFinalizeFrame>
	static typename TEnableIf<!HasFunc>::Type CallFinalizeFrameMemberFunc(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		// This is only fatal if we have NpState. If we had a simulation without NpState, a FinalizeFrame call isn't needed.
		npCheckf(!HasNpState(), TEXT("No FinalizeFrame implementation found. Implement DriverType::FinalizeFrame or ModelDef::FinalizeFrame"));
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	FinalizeSmoothingFrame
	//
	//	When a smoothing service is active, this is called every engine frame to push the final smoothed state to the driver. This is optional.
	// -----------------------------------------------------------------------------------------------------------------------------------
	static void FinalizeSmoothingFrame(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		CallFinalizeSmoothingFrameMemberFunc(Driver, SyncState, AuxState);
	}
	
	struct CFinalizeSmoothingFrameMemberFuncable
	{
		template <typename InDriverType, typename...>
		auto Requires(InDriverType* Driver, const SyncType* S, const AuxType* A) -> decltype(Driver->FinalizeSmoothingFrame(S, A));
	};

	static constexpr bool HasFinalizeSmoothingFrame = TModels_V<CFinalizeSmoothingFrameMemberFuncable, DriverType, SyncType, AuxType>;

	template<bool HasFunc=HasFinalizeSmoothingFrame>
	static typename TEnableIf<HasFunc>::Type CallFinalizeSmoothingFrameMemberFunc(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		npCheckSlow(Driver);
		Driver->FinalizeSmoothingFrame(SyncState, AuxState);
	}

	template<bool HasFunc=HasFinalizeSmoothingFrame>
	static typename TEnableIf<!HasFunc>::Type CallFinalizeSmoothingFrameMemberFunc(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		// FinalizeSmoothingFrame isn't required, but the driver/model won't get smoothed state
	}
	

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	RestoreFrame
	//
	//	Called prior to beginning rollback frames. This instance should put itself in whatever state it needs to be in for resimulation to
	//	run. In practice this should mean getting right collision+component states in sync so that any scene queries will get the correct
	//	data.
	// -----------------------------------------------------------------------------------------------------------------------------------

	static void RestoreFrame(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		FNetworkPredictionDriver<ModelDef>::CallRestoreFrameMemberFunc(Driver, SyncState, AuxState);
	}

	struct CRestoreFrameMemberFuncable
	{
		template <typename InDriverType, typename...>
		auto Requires(InDriverType* Driver, const SyncType* S, const AuxType* A) -> decltype(Driver->RestoreFrame(S, A));
	};

	static constexpr bool HasRestoreFrame = TModels_V<CRestoreFrameMemberFuncable, DriverType, SyncType, AuxType>;

	template<bool HasFunc=HasRestoreFrame>
	static typename TEnableIf<HasFunc>::Type CallRestoreFrameMemberFunc(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		npCheckSlow(Driver);
		Driver->RestoreFrame(SyncState, AuxState);
	}

	template<bool HasFunc=HasRestoreFrame>
	static typename TEnableIf<!HasFunc>::Type CallRestoreFrameMemberFunc(DriverType* Driver, const SyncType* SyncState, const AuxType* AuxState)
	{
		// This isn't a problem but we should probably do something if there is no RestoreFrame function:
		//	-Warn/complain (but user may not care in all cases. So may need a trait to opt out?)
		//	-Call FinalizeFrame: less boiler plate to add (but causes confusion and could lead to slow FinalizeFrames being called too often)
		//	-Force both Restore/Finalize Frame to be implemented but always implicitly call RestoreFrame before FinalizeFrame? (nah)
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	CallServerRPC
	//
	//	Tells the driver to call the Server RPC to send InputCmds to the server. UNetworkPredictionComponent::CallServerRPC is the default
	//	implementation and shouldn't need to be defined by the user.
	// -----------------------------------------------------------------------------------------------------------------------------------
		
	static void CallServerRPC(DriverType* Driver)
	{
		CallServerRPCMemberFunc(Driver);
	}
	
	struct CCallServerRPCMemberFuncable
	{
		template <typename InDriverType>
		auto Requires(InDriverType* Driver) -> decltype(Driver->CallServerRPC());
	};

	static constexpr bool HasCallServerRPC = TModels_V<CCallServerRPCMemberFuncable, DriverType>;

	template<bool HasFunc=HasCallServerRPC>
	static typename TEnableIf<HasFunc>::Type CallServerRPCMemberFunc(DriverType* Driver)
	{
		npCheckSlow(Driver);
		Driver->CallServerRPC();
	}

	template<bool HasFunc=HasCallServerRPC>
	static typename TEnableIf<!HasFunc>::Type CallServerRPCMemberFunc(DriverType* Driver)
	{
		npCheckf(false, TEXT("No CallServerRPC implementation found. Implement DriverType::CallServerRPC or ModelDef::CallServerRPC"));
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	Dispatch Cues
	//
	//	Forwards call to CueDispatche's DispatchCueRecord which will invoke the queued HandleCue events to the driver.
	// -----------------------------------------------------------------------------------------------------------------------------------
	
	template<typename InDriverType=DriverType>
	static void DispatchCues(TNetSimCueDispatcher<ModelDef>* CueDispatcher, InDriverType* Driver, int32 SimFrame, int32 SimTimeMS, const int32 FixedStepMS)
	{
		npCheckSlow(Driver);
		CueDispatcher-> template DispatchCueRecord<InDriverType>(*Driver, SimFrame, SimTimeMS, FixedStepMS);
	}

	static void DispatchCues(TNetSimCueDispatcher<ModelDef>* CueDispatcher, void* Driver, int32 SimFrame, int32 SimTimeMS, const int32 FixedStepMS)
	{
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	ShouldReconcile
	//
	//	Determines if Sync/Aux state have diverged enough to force a correction.
	//	The default implementation will call ShouldReconcile on the state itself: bool FMySyncState::ShouldReconcile(const FMySyncState& Authority) const.
	//  May also be defined as Driver member functions (separately for Synx/Aux):  bool MyDriver::ShouldReconcile(const FMySyncState* PredictedState, const FMySyncState* AuthorityState).
	// -----------------------------------------------------------------------------------------------------------------------------------
	static bool ShouldReconcile(DriverType* Driver, const TSyncAuxPair<StateTypes>& Predicted, const TSyncAuxPair<StateTypes>& Authority)
	{
		return CallShouldReconcileSyncStateMemberFunc(Driver, Predicted.Sync, Authority.Sync) ||
			   CallShouldReconcileAuxStateMemberFunc(Driver, Predicted.Aux, Authority.Aux);
	}

	struct CShouldReconcileSyncStateMemberFuncable
	{
		template <typename InDriverType, typename...>
		auto Requires(InDriverType* Driver, const SyncType* Predicted, const SyncType* Authority) -> decltype(
			Driver->ShouldReconcile(Predicted, Authority)
			);
	};

	struct CShouldReconcileAuxStateMemberFuncable
	{
		template <typename InDriverType, typename...>
		auto Requires(InDriverType* Driver, const AuxType* Predicted, const AuxType* Authority) -> decltype(
			Driver->ShouldReconcile(Predicted, Authority)
			);
	};

	
	static constexpr bool HasShouldReconcileSyncState = TModels_V<CShouldReconcileSyncStateMemberFuncable, DriverType, SyncType, SyncType>;
	static constexpr bool HasShouldReconcileAuxState  = TModels_V<CShouldReconcileAuxStateMemberFuncable,  DriverType, AuxType,  AuxType>;


	template<bool HasFunc = HasShouldReconcileSyncState>
	static typename TEnableIf<HasFunc, bool>::Type CallShouldReconcileSyncStateMemberFunc(DriverType* Driver, const SyncType* Predicted, const SyncType* Authority)
	{
		npCheckSlow(Driver);
		return Driver->ShouldReconcile(Predicted, Authority);
	}

	template<bool HasFunc = HasShouldReconcileSyncState>
	static typename TEnableIf<!HasFunc, bool>::Type CallShouldReconcileSyncStateMemberFunc(DriverType* Driver, const SyncType* Predicted, const SyncType* Authority)
	{
		// ShouldReconcile member function isn't required on the driver, as we'll fall back to querying the StateTypes directly
		return Predicted->ShouldReconcile(*Authority);
	}

	static bool CallShouldReconcileSyncStateMemberFunc(DriverType* Driver, const void* Predicted, const void* Authority)
	{
		return false;
	}


	template<bool HasFunc = HasShouldReconcileAuxState>
	static typename TEnableIf<HasFunc, bool>::Type CallShouldReconcileAuxStateMemberFunc(DriverType* Driver, const AuxType* Predicted, const AuxType* Authority)
	{
		npCheckSlow(Driver);
		return Driver->ShouldReconcile(Predicted, Authority);
	}

	template<bool HasFunc = HasShouldReconcileAuxState>
	static typename TEnableIf<!HasFunc, bool>::Type CallShouldReconcileAuxStateMemberFunc(DriverType* Driver, const AuxType* Predicted, const AuxType* Authority)
	{
		// ShouldReconcile member function isn't required on the driver, as we'll fall back to querying the StateTypes directly
		return Predicted->ShouldReconcile(*Authority);
	}

	static bool CallShouldReconcileAuxStateMemberFunc(DriverType* Driver, const void* Predicted, const void* Authority)
	{
		return false; 
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	Interpolate
	//
	//	Blend between From/To set of Sync/Aux states
	// -----------------------------------------------------------------------------------------------------------------------------------
	static void Interpolate(const TSyncAuxPair<StateTypes>& From, const TSyncAuxPair<StateTypes>& To, const float PCT, SyncType* SyncOut, AuxType* AuxOut)
	{
		InterpolateState(From.Sync, To.Sync, PCT, SyncOut);
		InterpolateState(From.Aux, To.Aux, PCT, AuxOut);
	}

	template<typename StateType>
	static void InterpolateState(const StateType* From, const StateType* To, const float PCT, StateType* Out)
	{
		Out->Interpolate(From, To, PCT);
	}
	
	static void InterpolateState(const void* From, const void* To, const float PCT, void* Out)
	{
	}

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	Show/Hide ForInterpolation
	//
	//	Interpolated sims are initially hidden until there are two valid states to interpolate between	
	// -----------------------------------------------------------------------------------------------------------------------------------

	static void SetHiddenForInterpolation(DriverType* Driver, bool bHide)
	{
		CallSetHiddenForInterpolation(Driver, bHide);
	}

	struct CSetHiddenForInterpolationFuncable
	{
		template <typename InDriverType>
		auto Requires(InDriverType* Driver, bool bHidden) -> decltype(Driver->SetHiddenForInterpolation(bHidden));
	};
	
	static constexpr bool HasSetHiddenForInterpolation = TModels_V<CSetHiddenForInterpolationFuncable, DriverType>;

	template<bool HasFunc=HasSetHiddenForInterpolation>
	static typename TEnableIf<HasFunc>::Type CallSetHiddenForInterpolation(DriverType* Driver, bool bHide)
	{
		npCheckSlow(Driver);
		Driver->SetHiddenForInterpolation(bHide);
	}

	template<bool HasFunc=HasSetHiddenForInterpolation>
	static typename TEnableIf<!HasFunc>::Type CallSetHiddenForInterpolation(DriverType* Driver, bool bHide)
	{
		CallSetHiddenForInterpolationFallback(Driver, bHide);
	}

	static void CallSetHiddenForInterpolationFallback(AActor* Driver, bool bHide)
	{
		Driver->SetActorHiddenInGame(bHide);
	}

	static void CallSetHiddenForInterpolationFallback(UActorComponent* Driver, bool bHide)
	{
		Driver->GetOwner()->SetActorHiddenInGame(bHide);
	}	


	// -----------------------------------------------------------------------------------------------------------------------------------
	//	ToString
	//
	//	Utility functions for turning user state into strings. User states can should define ToString(FAnsiStringBuilderBase&) and append
	//	*ANSI* strings to the builder. (E.g, don't use the TEXT macro).	
	//
	//	Ansi was chosen because for Trace purposes:
	//	We want tracing user state strings to be as fast and efficient as possible for Insights so that it can be enabled during development.
	//
	//	Logging is primary intended as a last resort for printf style debugging. The system should not output UserStates to the log in 
	//	normal circumstances. (Opting in via cvars or verbose logging categories would be ok).
	//
	//	If you really need to return the string's produced here, use an FStringOutputDevice. Otherwise they will be stacked allocated
	// -----------------------------------------------------------------------------------------------------------------------------------
	static void LogUserStates(const TNetworkPredictionState<StateTypes>& UserStates, FOutputDevice& Ar=*GLog)
	{
		TAnsiStringBuilder<512> Builder;

		Builder << '\n';
		ToString((InputType*)UserStates.Cmd, Builder);
		Builder << '\n';		
		ToString((SyncType*)UserStates.Sync, Builder);
		Builder << '\n';
		ToString((AuxType*)UserStates.Aux, Builder);

		Ar.Log(StringCast<TCHAR>(Builder.ToString()).Get());
	}

	template<typename StateType>
	static void LogUserState(const StateType* State, FOutputDevice& Ar=*GLog)
	{
		TAnsiStringBuilder<256> Builder;

		Builder << '\n';
		ToString(State, Builder);

		Ar.Log(StringCast<TCHAR>(Builder.ToString()).Get());
	}

	template<typename StateType>
	static void TraceUserStateString(const StateType* State, FAnsiStringBuilderBase& Builder)
	{
		ToString(State, Builder);
	}

	// Eventually: TraceUserStateBinary for Insights -> Editor debugging

	template<typename StateType>
	static void ToString(const StateType* State, FAnsiStringBuilderBase& Builder)
	{
		State->ToString(Builder);
	}

	static void ToString(const void* State, FAnsiStringBuilderBase& Builder) { }

	// -----------------------------------------------------------------------------------------------------------------------------------
	//	NetSerialize
	//
	//	Forwards NetSerialize call to the user type.
	// -----------------------------------------------------------------------------------------------------------------------------------
	template<typename StateType>
	static void NetSerialize(TConditionalState<StateType>& State, const FNetSerializeParams& P)
	{
		State->NetSerialize(P);
	}

	static void NetSerialize(TConditionalState<void>& State, const FNetSerializeParams& P) { }
};


// This is the actual template to specialize when wanting to override functions
template<typename ModelDef>
struct FNetworkPredictionDriver : FNetworkPredictionDriverBase<ModelDef>
{

};
