// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NetworkPredictionModelDef.h"
#include "NetworkPredictionDriver.h"

struct FGenericKinematicActorSyncState
{
	FVector_NetQuantize100	Location;
	FQuat Rotation;
};

// Generic def for kinematic (non physics) actor that doesn't have a backing simulation. This is quite limited in what it can do,
// but hopefully useful is that they can still be recorded and restored
struct FGenericKinematicActorDef : FNetworkPredictionModelDef
{
	NP_MODEL_BODY();

	using StateTypes = TNetworkPredictionStateTypes<void, FGenericKinematicActorSyncState, void>;
	using Driver = AActor;
	static const TCHAR* GetName() { return TEXT("Generic Kinematic Actor"); }
	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::PreKinematicMovers; }
};

template<>
struct FNetworkPredictionDriver<FGenericKinematicActorDef> : FNetworkPredictionDriverBase<FGenericKinematicActorDef>
{
	static void InitializeSimulationState(AActor* ActorDriver, FGenericKinematicActorSyncState* Sync, void* Aux)
	{
		const FTransform& Transform = ActorDriver->GetActorTransform();
		Sync->Location = Transform.GetLocation();
		Sync->Rotation = Transform.GetRotation();
	}
};
