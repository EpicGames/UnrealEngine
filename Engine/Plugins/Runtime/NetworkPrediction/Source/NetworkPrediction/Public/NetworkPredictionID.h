// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// Unique server assigned ID for a network sim instance
struct FNetworkPredictionID
{
	FNetworkPredictionID() = default;
	explicit FNetworkPredictionID(int32 InSpawnID) : SpawnID(InSpawnID)
	{ 
		static int32 TraceIDCount=1;
		TraceID = ++TraceIDCount;
	}

	explicit FNetworkPredictionID(int32 InSpawnID, int32 InTraceID) : SpawnID(InSpawnID), TraceID(InTraceID) { }

	int32 GetTraceID() const { return TraceID; }

	bool IsValid() const { return SpawnID != INDEX_NONE; }

	operator int32() const { return SpawnID; }
	bool operator<  (const FNetworkPredictionID &rhs) const { return(this->SpawnID < rhs.SpawnID); }
	bool operator<= (const FNetworkPredictionID &rhs) const { return(this->SpawnID <= rhs.SpawnID); }
	bool operator>  (const FNetworkPredictionID &rhs) const { return(this->SpawnID > rhs.SpawnID); }
	bool operator>= (const FNetworkPredictionID &rhs) const { return(this->SpawnID >= rhs.SpawnID); }
	bool operator== (const FNetworkPredictionID &rhs) const { return(this->SpawnID == rhs.SpawnID); }
	bool operator!= (const FNetworkPredictionID &rhs) const { return(this->SpawnID != rhs.SpawnID); }

private:
	int32 SpawnID=INDEX_NONE;
	int32 TraceID=0;
};