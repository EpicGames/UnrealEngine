// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NetworkPredictionStateTypes.h"

struct FNetworkPredictionStateView;

// Arbitrary sort values used by system default definitions
enum class ENetworkPredictionSortPriority : uint8
{
	First = 1,
	PreKinematicMovers = 50,
	KinematicMovers = 75,
	PostKinematicMovers = 100,

	Last = 250
};

using FModelDefId = int32;
#define NP_MODEL_BODY() static FModelDefId ID

struct FNetworkPredictionModelDef
{
	// Actual defs should have:
	// NP_MODEL_BODY(); 

	// TNetworkPredictionStateTypes: User State Types (Input, Sync, Aux)
	// Enables: Reconcile, Ticking, Input, Finalize
	using StateTypes = TNetworkPredictionStateTypes<void,void,void>;

	// Object that runs SimulationTick
	// Requires: valid StateTypes
	// Enables: Ticking
	using Simulation = void;

	// Object class that can take output from prediction system. E.g AActor, AMyPawn.
	// See notes in NetworkPredictionDriver.h
	// Requires: StateTypes
	// Enables: Finalize, Cues
	using Driver = void;

	static const TCHAR* GetName() { return nullptr; }

	static constexpr int32 GetSortPriority() { return (int32)ENetworkPredictionSortPriority::Last; }
};

// ----------------------------------------------------------------------

template<typename ModelDef=FNetworkPredictionModelDef, typename SimulationType=typename ModelDef::Simulation>
struct FConditionalSimulationPtr
{
	FConditionalSimulationPtr(SimulationType* Sim=nullptr) : Simulation(Sim) { }
	SimulationType* operator->() const { return Simulation; }

private:
	SimulationType* Simulation = nullptr;
};

template<typename ModelDef>
struct FConditionalSimulationPtr<ModelDef, void>
{
	FConditionalSimulationPtr(void* v=nullptr) { }
};

// ----------------------------------------------------------------------

template<typename ModelDef=FNetworkPredictionModelDef>
struct TNetworkPredictionModelInfo
{
	using SimulationType = typename ModelDef::Simulation;
	using DriverType = typename ModelDef::Driver;

	FConditionalSimulationPtr<ModelDef> Simulation;			// Object that ticks this instance
	DriverType* Driver = nullptr;							// Object that handles input/out
	struct FNetworkPredictionStateView* View = nullptr;		// Game side view of state to update

	TNetworkPredictionModelInfo(SimulationType* InSimulation=nullptr, DriverType* InDriver=nullptr, FNetworkPredictionStateView* InView=nullptr)
		: Simulation(InSimulation), Driver(InDriver), View(InView) { }
};
