// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "NetworkPredictionLog.h"
#include "NetworkPredictionModelDef.h"
#include "NetworkPredictionCheck.h"

// Basic ModelDef type registry. ModelDefs are registered and assigned an ID (their ModelDef::ID) based on ModelDef::SortPriority.
// This ID is used as indices into the various NP service arrays
class FNetworkPredictionModelDefRegistry
{
	// Declaring this as private to help avoid sidestepping singleton
	FNetworkPredictionModelDefRegistry() = default;
	
public:

	NETWORKPREDICTION_API static FNetworkPredictionModelDefRegistry& Get();

	template<typename ModelDef>
	void RegisterType()
	{
		bFinalized = false; // New type must re-finalize

		if (!npEnsure(ModelDefList.Contains(&ModelDef::ID) == false))
		{
			return;
		}

		FTypeInfo TypeInfo =
		{
			&ModelDef::ID,				 // Must include NP_MODEL_BODY()
			ModelDef::GetSortPriority(), // Must implement ::GetSortPriorty()
			ModelDef::GetName()			 // Must implement ::GetName()
		};

		ModelDefList.Emplace(TypeInfo);
	}

	void FinalizeTypes();

private:
	struct FTypeInfo
	{
		bool operator==(const FModelDefId* OtherIDPtr) const
		{
			return IDPtr == OtherIDPtr;
		}

		FModelDefId* IDPtr = nullptr;
		int32 SortPriority;
		const TCHAR* Name;
	};

	TArray<FTypeInfo> ModelDefList;
	
	bool bFinalized = false;
};

template<typename ModelDef>
struct TNetworkPredictionModelDefRegisterHelper
{
	TNetworkPredictionModelDefRegisterHelper()
	{
		FNetworkPredictionModelDefRegistry::Get().RegisterType<ModelDef>();
	}
};

// Helper to register ModelDef type.
// Sets static ID to 0 (invalid) and calls global registration function
#define NP_MODEL_REGISTER(X) \
	FModelDefId X::ID=0; \
	static TNetworkPredictionModelDefRegisterHelper<X> NetModelAr_##X = TNetworkPredictionModelDefRegisterHelper<X>();
