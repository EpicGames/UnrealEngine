// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionID.h"
#include "NetworkPredictionStateView.h"
#include "NetworkPredictionConfig.h"
#include "NetworkPredictionCheck.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/EngineTypes.h"
#include "NetworkPredictionProxy.generated.h"

// -------------------------------------------------------------------------------------------------
//	FNetworkPredictionProxy
//
//	Proxy struct for interfacing with the NetworkPrediction system. 
//
//	Call Init<YourModelDef>(...) to bind to the system. Config(...) will change the current settings.
//	Include NetworkPredictionProxyInit.h in your cpp file to do this. (Don't include it in your class's header).
//
// -------------------------------------------------------------------------------------------------

struct FReplicationProxySet;
class UNetworkPredictionWorldManager;

UENUM()
enum class ENetworkPredictionStateRead
{
	// The authoritative, networked state values.
	Simulation,
	// The local "smoothed" or "corrected" state values. If no explicit presentation value is set, Simulation value is implied.
	// Presentation values never feed back into the simulation.
	Presentation,
};

USTRUCT()
struct FNetworkPredictionProxy
{
	GENERATED_BODY();

	// Parameter struct to be used with the Init function overload below, when a UWorld isn't available, such as in unit tests.
	template<typename ModelDef>
	struct FInitParams
	{
		UNetworkPredictionWorldManager* WorldManager;
		ENetMode Mode;
		const FReplicationProxySet& RepProxies;
		typename ModelDef::Simulation* Simulation = nullptr;
		typename ModelDef::Driver* Driver = nullptr;
	};

	// The init function that you need to call. This is defined in NetworkPredictionProxyInit.h (which should only be included by your .cpp file)
	template<typename ModelDef>
	void Init(UWorld* World, const FReplicationProxySet& RepProxies, typename ModelDef::Simulation* Simulation=nullptr, typename ModelDef::Driver* Driver=nullptr);

	template<typename ModelDef>
	void Init(const FInitParams<ModelDef>& Params);

	// When network role changes, initializes role storage and logic controller
	void InitForNetworkRole(ENetRole Role, bool bHasNetConnection)
	{
		CachedNetRole = Role;
		bCachedHasNetConnection = bHasNetConnection;
		if (ConfigFunc)
		{
			ConfigFunc(this, FNetworkPredictionID(), EConfigAction::UpdateConfigWithDefault);
		}
	}

	// Should only be called on the authority. Changes what how this instance is allowed to be configured
	void SetArchetype(const FNetworkPredictionInstanceArchetype& Archetype, const FNetworkPredictionInstanceConfig& Config)
	{
		ArchetypeDirtyCount++;
		Configure(Config);
	}

	// Call to change local configuration of proxy. Not networked.
	void Configure(const FNetworkPredictionInstanceConfig& Config)
	{
		CachedConfig = Config;
		if (ConfigFunc)
		{
			ConfigFunc(this, FNetworkPredictionID(), EConfigAction::None);
		}
	}

	// Unregisters from NetworkPrediction System
	void EndPlay()
	{
		if (ConfigFunc)
		{
			ConfigFunc(this, FNetworkPredictionID(), EConfigAction::EndPlay);
		}
	}

	// --------------------------------------------------------------------------------------------------------------------------
	//	Read/Write access for the current states (these are the states that will be used as input into next simulation frame
	// --------------------------------------------------------------------------------------------------------------------------

	// Returns pending InputCmd. (Note there is no Presentation version of InputCmds)
	template<typename TInputCmd>
	const TInputCmd* ReadInputCmd() const
	{
		return static_cast<TInputCmd*>(View.PendingInputCmd);
	}

	// Returns Presentation SyncState by default, if it is set. Otherwise returns pending Simulation value.
	template<typename TSyncState>
	const TSyncState* ReadSyncState(ENetworkPredictionStateRead ReadType = ENetworkPredictionStateRead::Presentation) const
	{
		if (ReadType == ENetworkPredictionStateRead::Presentation && View.PresentationSyncState)
		{
			return static_cast<TSyncState*>(View.PresentationSyncState);
		}

		return static_cast<TSyncState*>(View.PendingSyncState);
	}

	// Returns Prev Presentation SyncState by default, if it is set. Otherwise returns pending Simulation value.
	template<typename TSyncState>
	const TSyncState* ReadPrevPresentationSyncState() const
	{
		
		if (View.PrevPresentationSyncState)
		{
			return static_cast<TSyncState*>(View.PrevPresentationSyncState);
		}
		
		return static_cast<TSyncState*>(View.PendingSyncState);
	}

	// Returns Presentation AuxState by default, if it is set. Otherwise returns pending Simulation value.
	template<typename TAuxState>
	const TAuxState* ReadAuxState(ENetworkPredictionStateRead ReadType = ENetworkPredictionStateRead::Presentation) const
	{
		if (ReadType == ENetworkPredictionStateRead::Presentation && View.PresentationAuxState)
		{
			return static_cast<TAuxState*>(View.PresentationAuxState);
		}
		
		return static_cast<TAuxState*>(View.PendingAuxState);
	}

	// Returns Prev Presentation AuxState by default, if it is set. Otherwise returns pending Simulation value.
	template<typename TAuxState>
	const TAuxState* ReadPrevPresentationAuxState() const
	{
		if (View.PrevPresentationAuxState)
		{
			return static_cast<TAuxState*>(View.PrevPresentationAuxState);
		}
		
		return static_cast<TAuxState*>(View.PendingAuxState);
	}

	// Writes - must include NetworkPredictionProxyWrite.h in places that call this
	// Note that writes are implicitly done on the simulation state. It is not valid to modify the presentation value out of band.
	template<typename TInputCmd>
	const TInputCmd* WriteInputCmd(TFunctionRef<void(TInputCmd&)> WriteFunc, const FAnsiStringView& TraceMsg=FAnsiStringView());

	template<typename TSyncState>
	const TSyncState* WriteSyncState(TFunctionRef<void(TSyncState&)> WriteFunc, const FAnsiStringView& TraceMsg=FAnsiStringView());
	
	template<typename TSyncState>
	const TSyncState* WritePresentationSyncState(TFunctionRef<void(TSyncState&)> WriteFunc, const FAnsiStringView& TraceMsg=FAnsiStringView());
	
	template<typename TSyncState>
	const TSyncState* WritePrevPresentationSyncState(TFunctionRef<void(TSyncState&)> WriteFunc, const FAnsiStringView& TraceMsg=FAnsiStringView());

	template<typename TAuxState>
	const TAuxState* WriteAuxState(TFunctionRef<void(TAuxState&)> WriteFunc, const FAnsiStringView& TraceMsg=FAnsiStringView());

	template<typename TAuxState>
	const TAuxState* WritePresentationAuxState(TFunctionRef<void(TAuxState&)> WriteFunc, const FAnsiStringView& TraceMsg=FAnsiStringView());

	template<typename TAuxState>
	const TAuxState* WritePrevPresentationAuxState(TFunctionRef<void(TAuxState&)> WriteFunc, const FAnsiStringView& TraceMsg=FAnsiStringView());
	

	// ------------------------------------------------------------------------------------

	FNetSimCueDispatcher* GetCueDispatcher() const
	{
		return View.CueDispatcher;
	}

	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
	{
		uint32 RawID=(uint32)ID;
		Ar.SerializeIntPacked(RawID);
		CachedArchetype.NetSerialize(Ar);

		if (Ar.IsLoading())
		{
			ConfigureAfterLoading(RawID);
		}

		bOutSuccess = !Ar.IsError();
		return true;
	}

	void ConfigureAfterLoading(uint32 LoadedID)
	{
		if ((int32)ID != LoadedID)
		{
			if (ConfigFunc)
			{
				// We've already been init, so have to go through ConfigFunc to remap new id
				FNetworkPredictionID NewID(LoadedID, ID.GetTraceID());
				ConfigFunc(this, NewID, EConfigAction::UpdateConfigWithDefault);
			}
			else
			{
				// haven't been init yet so just set the replicated ID so we don't create a client side one
				ID = FNetworkPredictionID(LoadedID);
			}
		}
		else
		{
			if (npEnsureMsgf((ConfigFunc != nullptr), TEXT("Attempting to configure with a changed archetype but no ConfigFunc has been set yet!")))
			{
				// Archetype change, call ConfigFunc but don't change ID
				ConfigFunc(this, FNetworkPredictionID(), EConfigAction::UpdateConfigWithDefault);
			}
		}
	}

	bool Identical(const FNetworkPredictionProxy* Other, uint32 PortFlags) const
	{
		return ID == Other->ID && ArchetypeDirtyCount == Other->ArchetypeDirtyCount;
	}

	// ------------------------------------------------------------------------------------

	int32 GetPendingFrame() const { return View.PendingFrame; }
	int32 GetTotalSimTimeMS() const { return View.SimTimeMS; }
	ENetRole GetCachedNetRole() const { return CachedNetRole; }
	bool GetCachedHasNetConnection() const { return bCachedHasNetConnection; }

public:	// Direct accessors for Iris replication support only
	FNetworkPredictionID& GetID() { return ID; }
	FNetworkPredictionInstanceArchetype& GetCachedArchetype() { return CachedArchetype; }

private:

	// Allows ConfigFunc to be invoked to "do a thing" instead of set a new config/id.
	// This is useful because ConfigFunc can make the untemplated caller -> ModelDef jump.
	enum class EConfigAction : uint8
	{
		None,
		EndPlay,
		UpdateConfigWithDefault,
		TraceInput,
		TraceSync,
		TraceAux
	};

	void TraceViaConfigFunc(EConfigAction Action);

	FNetworkPredictionID ID;
	FNetworkPredictionStateView View;

	ENetRole CachedNetRole = ROLE_None;
	bool bCachedHasNetConnection = false;
	FNetworkPredictionInstanceConfig CachedConfig;
	FNetworkPredictionInstanceArchetype CachedArchetype;
	uint8 ArchetypeDirtyCount = 0;

	TFunction<void(FNetworkPredictionProxy* const, FNetworkPredictionID NewID, EConfigAction Action)>	ConfigFunc;

	UPROPERTY()
	TObjectPtr<UNetworkPredictionWorldManager> WorldManager = nullptr;
};

template<>
struct TStructOpsTypeTraits<FNetworkPredictionProxy> : public TStructOpsTypeTraitsBase2<FNetworkPredictionProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdentical = true,
	};
};
