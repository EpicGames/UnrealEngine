// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "NetworkPredictionProxy.h"
#include "NetworkPredictionWorldManager.h"

struct FNetworkPredictionSettings;

// The init function binds to the templated methods on UNetworkPredictionMAnager. This will "bring in" all the templated systems on NP, so this file should only be 
// included in your .cpp file that is calling Init.
template<typename ModelDef>
void FNetworkPredictionProxy::Init(const FInitParams<ModelDef>& Params)
{
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	// Aquire an ID but don't register yet
	WorldManager = Params.WorldManager;
	npCheckSlow(WorldManager);

	if (ID.IsValid() == false)
	{
		// Brand new registration. Initialize the default Archetype for this ModelDef
		if (!FNetworkPredictionDriver<ModelDef>::GetDefaultArchetype(CachedArchetype, WorldManager->PreferredDefaultTickingPolicy()))
		{
			UE_LOGF(LogNetworkPrediction, Error, "Unable to initialize DefaultArchetype. Skipping registration with NetworkPrediction");
			return;
		}

		// Assign ID. Client will assign a temporary ID that later gets remapped via a call to ConfigFunc --> RemapClientSimulationID
		ID = WorldManager->CreateSimulationID(Params.Mode == NM_Client);
	}

	WorldManager->RegisterInstance<ModelDef>(ID, TNetworkPredictionModelInfo<ModelDef>(Params.Simulation, Params.Driver, &View));

	const FReplicationProxySet& RepProxies = Params.RepProxies;
	ConfigFunc = [RepProxies](FNetworkPredictionProxy* const This, FNetworkPredictionID NewID, EConfigAction Action)
	{
		if (This->WorldManager == nullptr)
		{
			return;
		}

		switch (Action)
		{
			case EConfigAction::EndPlay:
				This->WorldManager->UnregisterInstance<ModelDef>(This->ID);
				return;

			case EConfigAction::UpdateConfigWithDefault:
				npEnsureSlow(This->CachedNetRole != ROLE_None); // role must have already been set
				This->CachedConfig = FNetworkPredictionDriver<ModelDef>::GetConfig(This->CachedArchetype, This->WorldManager->GetSettings(), This->CachedNetRole, This->bCachedHasNetConnection);
				break; // purposefully breaking, not returning, so that we do call ConfigureInstance
				
			case EConfigAction::TraceInput:
				UE_NP_TRACE_USER_STATE_INPUT(ModelDef, (InputType*)This->View.PendingInputCmd);
				return;

			case EConfigAction::TraceSync:
				UE_NP_TRACE_USER_STATE_SYNC(ModelDef, (SyncType*)This->View.PendingSyncState);
				return;

			case EConfigAction::TraceAux:
				UE_NP_TRACE_USER_STATE_AUX(ModelDef, (AuxType*)This->View.PendingAuxState);
				return;
		};

		if (NewID.IsValid())
		{
			This->WorldManager->RemapClientSimulationID<ModelDef>(This->ID, NewID);
			This->ID = NewID;
		}

		if (This->CachedNetRole != ROLE_None && (int32)This->ID > 0) // Don't configure until NetRole and server-assigned ID are present
		{
			This->WorldManager->ConfigureInstance<ModelDef>(This->ID, This->CachedArchetype, This->CachedConfig, RepProxies, This->CachedNetRole, This->bCachedHasNetConnection);
		}
	};
}

template<typename ModelDef>
void FNetworkPredictionProxy::Init(UWorld* World, const FReplicationProxySet& RepProxies, typename ModelDef::Simulation* Simulation, typename ModelDef::Driver* Driver)
{
	FNetworkPredictionProxy::FInitParams<ModelDef> Params = {World->GetSubsystem<UNetworkPredictionWorldManager>(), World->GetNetMode(), RepProxies, Simulation, Driver};
	Init<ModelDef>(Params);
}
