// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionProxy.h"
#include "NetworkPredictionTrace.h"

inline void FNetworkPredictionProxy::TraceViaConfigFunc(EConfigAction Action)
{
	// The ConfigFunc allows use to use the registered ModelDef type to access FNetworkPredictionDriver<ModelDef>::TraceUserState
	// this allows for per-ModelDef customizations but more importantly will call State->ToString on the correct child class.
	// consider FChildSyncState : FBaseSyncState{}; with a base driver class that calls WriteSyncState<FBaseSyncState>(...);
#if UE_NP_TRACE_USER_STATES_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(NetworkPredictionChannel))
	{
		ConfigFunc(this, FNetworkPredictionID(), Action);
	}
#endif
}

template<typename TInputCmd>
const TInputCmd* FNetworkPredictionProxy::WriteInputCmd(TFunctionRef<void(TInputCmd&)> WriteFunc,const FAnsiStringView& TraceMsg)
{
	if (TInputCmd* InputCmd = static_cast<TInputCmd*>(View.PendingInputCmd))
	{
		WriteFunc(*InputCmd);
		
		UE_NP_TRACE_OOB_STATE_MOD(ID.GetTraceID(), View.PendingFrame, TraceMsg);
		TraceViaConfigFunc(EConfigAction::TraceInput);
		return InputCmd;
	}
	return nullptr;
}

template<typename TSyncState>
const TSyncState* FNetworkPredictionProxy::WriteSyncState(TFunctionRef<void(TSyncState&)> WriteFunc, const FAnsiStringView& TraceMsg)
{
	if (TSyncState* SyncState = static_cast<TSyncState*>(View.PendingSyncState))
	{
		WriteFunc(*SyncState);
		UE_NP_TRACE_OOB_STATE_MOD(ID.GetTraceID(), View.PendingFrame, TraceMsg);
		ConfigFunc(this, FNetworkPredictionID(), EConfigAction::TraceSync);
		return SyncState;
	}
	return nullptr;
}

template<typename TSyncState>
const TSyncState* FNetworkPredictionProxy::WritePresentationSyncState(TFunctionRef<void(TSyncState&)> WriteFunc, const FAnsiStringView& TraceMsg)
{
	if (TSyncState* SyncState = static_cast<TSyncState*>(View.PresentationSyncState))
	{
		WriteFunc(*SyncState);
		UE_NP_TRACE_OOB_STATE_MOD(ID.GetTraceID(), View.PendingFrame, TraceMsg);
		ConfigFunc(this, FNetworkPredictionID(), EConfigAction::TraceSync);
		return SyncState;
	}
	return nullptr;
}

template<typename TSyncState>
const TSyncState* FNetworkPredictionProxy::WritePrevPresentationSyncState(TFunctionRef<void(TSyncState&)> WriteFunc, const FAnsiStringView& TraceMsg)
{
	if (TSyncState* SyncState = static_cast<TSyncState*>(View.PrevPresentationSyncState))
	{
		WriteFunc(*SyncState);
		UE_NP_TRACE_OOB_STATE_MOD(ID.GetTraceID(), View.PendingFrame, TraceMsg);
		ConfigFunc(this, FNetworkPredictionID(), EConfigAction::TraceSync);
		return SyncState;
	}
	return nullptr;
}

template<typename TAuxState>
const TAuxState* FNetworkPredictionProxy::WriteAuxState(TFunctionRef<void(TAuxState&)> WriteFunc, const FAnsiStringView& TraceMsg)
{
	if (TAuxState* AuxState = static_cast<TAuxState*>(View.PendingAuxState))
	{
		WriteFunc(*AuxState);
		UE_NP_TRACE_OOB_STATE_MOD(ID.GetTraceID(), View.PendingFrame, TraceMsg);
		ConfigFunc(this, FNetworkPredictionID(), EConfigAction::TraceAux);
		return AuxState;
	}
	return nullptr;
}

template<typename TAuxState>
const TAuxState* FNetworkPredictionProxy::WritePresentationAuxState(TFunctionRef<void(TAuxState&)> WriteFunc, const FAnsiStringView& TraceMsg)
{
	if (TAuxState* AuxState = static_cast<TAuxState*>(View.PresentationAuxState))
	{
		WriteFunc(*AuxState);
		UE_NP_TRACE_OOB_STATE_MOD(ID.GetTraceID(), View.PendingFrame, TraceMsg);
		ConfigFunc(this, FNetworkPredictionID(), EConfigAction::TraceAux);
		return AuxState;
	}
	return nullptr;
}

template<typename TAuxState>
const TAuxState* FNetworkPredictionProxy::WritePrevPresentationAuxState(TFunctionRef<void(TAuxState&)> WriteFunc, const FAnsiStringView& TraceMsg)
{
	if (TAuxState* AuxState = static_cast<TAuxState*>(View.PrevPresentationAuxState))
	{
		WriteFunc(*AuxState);
		UE_NP_TRACE_OOB_STATE_MOD(ID.GetTraceID(), View.PendingFrame, TraceMsg);
		ConfigFunc(this, FNetworkPredictionID(), EConfigAction::TraceAux);
		return AuxState;
	}
	return nullptr;
}