// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GameFramework/Actor.h"

#include "NetworkPredictionReplicatedManager.generated.h"

#define UE_API NETWORKPREDICTION_API

// This is a replicated "manager" for network prediction. Its purpose is only to replicate system-wide data that is not bound to an actor.
// Currently this is only to house a "mini packagemap" which allows stable shared indices that map to a small set of uobjects to be.
// UPackageMap can assign per-client net indices which invalidates sharing as well as forces 32 bit guis. this is a more specialzed case
// where we want to replicate IDs as btyes.

USTRUCT()
struct FSharedPackageMapItem
{
	GENERATED_BODY()

	UPROPERTY()
	TSoftObjectPtr<UObject> SoftPtr;
};

USTRUCT()
struct FSharedPackageMap
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FSharedPackageMapItem> Items;
};

UCLASS(MinimalAPI)
class ANetworkPredictionReplicatedManager : public AActor
{
	GENERATED_BODY()

public:

	UE_API ANetworkPredictionReplicatedManager();

	static UE_API FDelegateHandle OnAuthoritySpawn(const TFunction<void(ANetworkPredictionReplicatedManager*)>& Func);
	static void UnregisterOnAuthoritySpawn(FDelegateHandle Handle) { OnAuthoritySpawnDelegate.Remove(Handle); }

	UE_API virtual void BeginPlay();

	UE_API uint8 AddObjectToSharedPackageMap(TSoftObjectPtr<UObject> SoftPtr);

	UE_API uint8 GetIDForObject(UObject* Obj) const;

	UE_API TSoftObjectPtr<UObject> GetObjectForID(uint8 ID) const;

private:

	UPROPERTY(Replicated)
	FSharedPackageMap SharedPackageMap;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAuthoritySpawn, ANetworkPredictionReplicatedManager*)
	static UE_API FOnAuthoritySpawn OnAuthoritySpawnDelegate;

	static UE_API TWeakObjectPtr<ANetworkPredictionReplicatedManager> AuthorityInstance;
};

#undef UE_API
