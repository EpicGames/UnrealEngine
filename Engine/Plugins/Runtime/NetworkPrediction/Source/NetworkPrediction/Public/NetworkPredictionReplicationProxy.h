// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NetworkPredictionCheck.h"
#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"
#include "Iris/Serialization/IrisObjectReferencePackageMap.h"


#include "NetworkPredictionReplicationProxy.generated.h"

#define UE_API NETWORKPREDICTION_API

struct FNetworkPredictionProxy;
class UPackageMap;

// Target of replication
enum class EReplicationProxyTarget: uint8
{
	ServerRPC,			// Client -> Server
	AutonomousProxy,	// Owning/Controlling client
	SimulatedProxy,		// Non owning client
	Replay,				// Replay net driver
};

inline FString LexToString(EReplicationProxyTarget A)
{
	return *UEnum::GetValueAsString(TEXT("NetworkPrediction.EReplicationProxyTarget"), A);
}

// The parameters for NetSerialize that are passed around the system. Everything should use this, expecting to have to add more.
struct FNetSerializeParams
{
	FNetSerializeParams(FArchive& InAr, UPackageMap* InMap=nullptr) : Ar(InAr), Map(InMap) { }

	FArchive& Ar;
	UPackageMap* Map;
};


// Redirects NetSerialize to a dynamically set NetSerializeFunc.
// This is how we hook into the replication systems role-based serialization
USTRUCT()
struct FReplicationProxy
{
	GENERATED_BODY()

	UE_API void Init(FNetworkPredictionProxy* InNetSimProxy, EReplicationProxyTarget InReplicationTarget);
	UE_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);
	UE_API void OnPreReplication();	
	UE_API bool Identical(const FReplicationProxy* Other, uint32 PortFlags) const;

	int32 GetCachedPendingFrame() const 
	{ 
		return CachedPendingFrame; 
	}

	TFunction<void(const FNetSerializeParams& P)> NetSerializeFunc;

	FNetworkPredictionProxy* NetSimProxy = nullptr;



public:
	// Number of bits to reserve in serialization container. Make this large enough to try to avoid re-allocation during the worst case calls.
	static constexpr uint32 NumReservedBits = 4096;
	static constexpr uint32 MaxNumBits = 16384;

	// Iris support: holding ptr to associated actor for replication bridge / package map retrieval
	TWeakObjectPtr<AActor> OwnerActor = nullptr;

	// Iris support: TInlineAllocator used with TBitArray takes the number of 32-bit dwords, but the define is in number of bits, so convert here by dividing by 32.
	TBitArray<TInlineAllocator<NumReservedBits / NumBitsPerDWORD>> DataBits;

	// Iris support: Since this struct uses custom serialization path we need to explicitly capture exports in order to forward them to iris
	// This is managed by the use of a custom packagemap.
	UE::Net::FIrisPackageMapExports PackageMapExports;

	// Iris support: Since we capturing data outside of the normal serialization path we also need to store exports to inject when actually sending the data.
	UE::Net::FNetTokenExportContext::FNetTokenExports NetTokensPendingExport;

private:

	EReplicationProxyTarget ReplicationTarget;
	int32 CachedPendingFrame = INDEX_NONE;
};

template<>
struct TStructOpsTypeTraits<FReplicationProxy> : public TStructOpsTypeTraitsBase2<FReplicationProxy>
{
	enum
	{
		WithNetSerializer = true,
		WithIdentical = true,
	};
};

// Collection of each replication proxy
struct FReplicationProxySet
{
	FReplicationProxy* ServerRPC = nullptr;
	FReplicationProxy* AutonomousProxy = nullptr;
	FReplicationProxy* SimulatedProxy = nullptr;
	FReplicationProxy* Replay = nullptr;

	void UnbindAll() const
	{
		npCheckSlow(ServerRPC && AutonomousProxy && SimulatedProxy && Replay);
		ServerRPC->NetSerializeFunc = nullptr;
		AutonomousProxy->NetSerializeFunc = nullptr;
		SimulatedProxy->NetSerializeFunc = nullptr;
		Replay->NetSerializeFunc = nullptr;
	}
};



// -------------------------------------------------------------------------------------------------------------------------------
//	FServerRPCProxyParameter
//	Used for the client->Server RPC. Since this is instantiated on the stack by the replication system prior to net serializing,
//	we have no opportunity to point the RPC parameter to the member variables we want. So we serialize into a generic temp byte buffer
//	and move them into the real buffers on the component in the RPC body (via ::SerializeToProxy).
// -------------------------------------------------------------------------------------------------------------------------------
USTRUCT()
struct FServerReplicationRPCParameter
{
	GENERATED_BODY()

	// Receive flow: ctor() -> NetSerialize
	FServerReplicationRPCParameter() : Proxy(nullptr)	{ }

	// Send flow: ctor(Proxy) -> SerializeToProxy
	FServerReplicationRPCParameter(FReplicationProxy& InProxy) : Proxy(&InProxy) { }
	void SetReplicationProxy(FReplicationProxy& InProxy) { Proxy = &InProxy; }
	UE_API bool SerializeToProxy(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);
	UE_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	UPackageMap* GetPackageMap() const
	{ 
		return CachedPackageMap; 
	}

public:
	// Iris support
	static UIrisObjectReferencePackageMap* GetIrisPackageMapToCaptureReferences(UNetConnection* NetConnection, UE::Net::FIrisPackageMapExports& PackageMapExports);
	static UIrisObjectReferencePackageMap* GetIrisPackageMapToReadReferences(const UNetConnection* NetConnection, UE::Net::FIrisPackageMapExports& PackageMapExports);

	// Number of bits to reserve in serialization container. Make this large enough to try to avoid re-allocation during worst-case RPC calls.
	static constexpr uint32 NumReservedBits = 4096;
	static constexpr uint32 MaxNumBits = 16384;

	// Iris support: TInlineAllocator used with TBitArray takes the number of 32-bit dwords, but the define is in number of bits, so convert here by dividing by 32.
	TBitArray<TInlineAllocator<NumReservedBits / NumBitsPerDWORD>> DataBits;

	// Iris support: Since this struct uses custom serialization path we need to explicitly capture exports in order to forward them to iris
	// This is managed by the use of a custom packagemap.
	UE::Net::FIrisPackageMapExports PackageMapExports;

	// Iris support: Since we capturing data outside of the normal serialization path we also need to store exports to inject when actually sending the data.
	UE::Net::FNetTokenExportContext::FNetTokenExports NetTokensPendingExport;

private:
	FReplicationProxy* Proxy = nullptr;
	UPackageMap* CachedPackageMap = nullptr;	// Used for non-Iris replication
};

template<>
struct TStructOpsTypeTraits<FServerReplicationRPCParameter> : public TStructOpsTypeTraitsBase2<FServerReplicationRPCParameter>
{
	enum
	{
		WithNetSerializer = true
	};
};

// Helper struct to bypass the bandwidth limit imposed by the engine's NetDriver (QueuedBits, NetSpeed, etc).
// This is really a temp measure to make the system easier to drop in/try in a project without messing with your engine settings.
// (bandwidth optimizations have not been done yet and the system in general hasn't been stressed with packetloss / gaps in command streams)
// So, you are free to use this in your own code but it may be removed one day. Hopefully in general bandwidth limiting will also become more robust.
struct FScopedBandwidthLimitBypass
{
	UE_API FScopedBandwidthLimitBypass(AActor* OwnerActor);
	UE_API ~FScopedBandwidthLimitBypass();
private:

	int64 RestoreBits = 0;
	class UNetConnection* CachedNetConnection = nullptr;
};

#undef UE_API
