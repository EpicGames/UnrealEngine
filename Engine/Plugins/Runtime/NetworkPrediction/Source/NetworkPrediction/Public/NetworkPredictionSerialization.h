// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Net/UnrealNetwork.h" // For MakeRelative
#include "NetworkPredictionCVars.h"
#include "NetworkPredictionReplicationProxy.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionTrace.h"
#include "Services/NetworkPredictionInstanceData.h"
#include "UObject/CoreNet.h"

#define UE_API NETWORKPREDICTION_API


#define NETSIM_ENABLE_CHECKSUMS !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#if NETSIM_ENABLE_CHECKSUMS 
#define NETSIM_CHECKSUM(Ser) SerializeChecksum(Ser,0xA186A384, false);
#else
#define NETSIM_CHECKSUM
#endif

#ifndef NETSIM_NETCONSTANT_NUM_BITS_FRAME
#define NETSIM_NETCONSTANT_NUM_BITS_FRAME 8	// Allows you to override this setting via UBT, but access via cleaner FActorMotionNetworkingConstants::NUM_BITS_FRAME
#endif

namespace NetworkPredictionCVars
{
	NETSIM_DEVCVAR_SHIPCONST_INT(ForceSendDefaultInputCommands, 0, "np.ForceSendDefaultInputCommands", "While enabled on a client, it will send default input cmds to the server, rather than the locally-produced input. This is a way to introduce de-syncs/rollbacks for debugging.");

	// Options governing fixed-tick time dilation. See CVar definitions for descriptions.
	extern UE_API TAutoConsoleVariable<bool> EnableTimeDilation;
	extern UE_API TAutoConsoleVariable<float> TimeDilationAmount;
	extern UE_API TAutoConsoleVariable<bool> AllowTimeDilationEscalation;
	extern UE_API TAutoConsoleVariable<float> TimeDilationEscalationDecay;
	extern UE_API TAutoConsoleVariable<float> TimeDilationEscalationDecayMax;
	extern UE_API TAutoConsoleVariable<float> TimeDilationMax;
	extern UE_API TAutoConsoleVariable<float> TimeDilationMin;

}

struct FNetworkPredictionSerialization
{
	// How many bits we use to NetSerialize Frame numbers. This is only relevant AP Client <--> Server communication.
	// Frames are stored locally as 32 bit integers, but we use a smaller number of bits to NetSerialize.
	// The system internally guards from Frame numbers diverging. E.g, the client will not generate new frames if the
	// last serialization frame would be pushed out of the buffer. Server does not generate frames without input from client.
	enum { NUM_BITS_FRAME = NETSIM_NETCONSTANT_NUM_BITS_FRAME };

	// Abs max value we encode into the bit writer
	enum { MAX_FRAME_WRITE = 1 << NUM_BITS_FRAME };

	// This is the threshold at which we would wrap around and incorrectly assign a frame on the receiving side.
	// E.g, If there are FRAME_ERROR_THRESHOLD frames that do not make it across from sender->receiver, the
	// receiver will have incorrect local values. With 8 bits, this works out to be 128 frames or about 2 seconds at 60fps.
	enum { FRAME_ERROR_THRESHOLD = MAX_FRAME_WRITE / 2};

	// Helper to serialize an int32 frame as 8 bits. Returns the unpacked value (this will be same as input in the save path)
	static int32 SerializeFrame(FArchive& Ar, int32 Frame, int32 RelativeFrame)
	{
		if (Ar.IsSaving())
		{
			((FNetBitWriter&)Ar).WriteIntWrapped( Frame, MAX_FRAME_WRITE );
			return Frame;
		}

		return MakeRelative(((FNetBitReader&)Ar).ReadInt( MAX_FRAME_WRITE ), RelativeFrame, MAX_FRAME_WRITE );
	}

	// Disabled right now: this is causing issues with JIP
	static void WriteCompressedFrame(FArchive& Ar, int32 Frame)
	{
		Ar << Frame;

		//npCheckSlow(Ar.IsSaving());
		//((FNetBitWriter&)Ar).WriteIntWrapped( Frame, MAX_FRAME_WRITE );
	}

	// Disabled right now: this is causing issues with JIP
	static int32 ReadCompressedFrame(FArchive& Ar, int32 RelativeFrame)
	{
		int32 Frame = 0;
		Ar << Frame;
		return Frame;

		//const int32 SerializedInt = ((FNetBitReader&)Ar).ReadInt( MAX_FRAME_WRITE );
		//return MakeRelative(SerializedInt, RelativeFrame, MAX_FRAME_WRITE );
	}

	// For serializing timestamps
	static void SerializeTimeMS(FArchive& Ar, int32& TimestampMS)
	{
		// if this shows up in profiles, we may be able to do a MakeRelative scheme like frames
		Ar << TimestampMS;
	}

	// For serializing DeltaMS, expected to be small (< 1000)
	static void SerializeDeltaMS(FArchive& Ar, int32& DeltaTimeMS)
	{
		Ar.SerializeIntPacked((uint32&)DeltaTimeMS);
	}
};

// ---------------------------------------------------------------------------------------------------------------------------
//	AP Client -> Server replication
//
//	The Fixed/Independent ticking implementations are a more than trivially different so they are split into separate implementations.
//	Both currently send the last 'NumInputsPerSend' per serialization, determined from config settings. This could be improved with something more dynamic.
//
// ---------------------------------------------------------------------------------------------------------------------------

template<typename InModelDef>
class TFixedTickReplicator_Server
{
public:

	using ModelDef = InModelDef;
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;

	static void SetNumInputsPerSend(int32 NumToSend) 
	{ 
		NumInputsPerSend = NumToSend; 
	}

	// ------------------------------------------------------------------------------------------------------------
	// TFixedTickReplicator_Server::NetRecv Server Receiving from AP client
	// ------------------------------------------------------------------------------------------------------------
	static void NetRecv(const FNetSerializeParams& P, TServerRecvData_Fixed<ModelDef>& ServerRecvData, TModelDataStore<ModelDef>* DataStore, FFixedTickState* TickState)
	{
		FArchive& Ar = P.Ar;
		NETSIM_CHECKSUM(Ar);

		const int32 EndFrame = FNetworkPredictionSerialization::ReadCompressedFrame(Ar, ServerRecvData.LastRecvFrame); // 1. StartFrame
		const int32 StartFrame = EndFrame - NumInputsPerSend;
		const bool bIsStarvedForInput = ServerRecvData.LastRecvFrame <= ServerRecvData.LastConsumedFrame;

		for (int32 Frame=StartFrame; Frame < EndFrame; ++Frame)
		{
			if (Frame <= ServerRecvData.LastRecvFrame || Frame <= ServerRecvData.LastConsumedFrame)
			{
				EatCmd(P);
			}
			else
			{
				if (!bIsStarvedForInput)
				{
					for (int32 DroppedFrame = ServerRecvData.LastRecvFrame+1; DroppedFrame < Frame; ++DroppedFrame)
					{
						UE_NP_TRACE_SYSTEM_FAULT("Gap in input stream detected on server. Client frames involved: LastConsumedFrame: %d LastRecvFrame: %d. DroppedFrame: %d", ServerRecvData.LastConsumedFrame, ServerRecvData.LastRecvFrame, DroppedFrame);
						if (DroppedFrame > 0)
						{
							// FixedTick can't skip frames like independent, so copy previous input
							ServerRecvData.InputBuffer[DroppedFrame] = ServerRecvData.InputBuffer[DroppedFrame-1];
						}
					}
				}
				else
				{

					if (!NetworkPredictionCVars::EnableTimeDilation->GetBool())
					{
						UE_NP_TRACE_SYSTEM_FAULT("Recovering from input stream starvation on server. Advancing over dropped client frames [%d-%d]", ServerRecvData.LastConsumedFrame + 1, StartFrame - 1);
						ServerRecvData.LastConsumedFrame = StartFrame - 1;
					}
					else
					{
						UE_NP_TRACE_SYSTEM_FAULT("Recovering from input stream starvation on server. Advancing over dropped client frame %d", ServerRecvData.LastConsumedFrame + 1);
					}
				}

				npEnsure(Frame >= 0);

				FNetworkPredictionDriver<ModelDef>::NetSerialize(ServerRecvData.InputBuffer[Frame], P); // 2. InputCmd

				ServerRecvData.LastRecvFrame = Frame;

				// Trace what we received
				const int32 ExpectedFrameDelay = ServerRecvData.LastRecvFrame - ServerRecvData.LastConsumedFrame;
				const int32 ExpectedConsumeFrame = TickState->PendingFrame + ExpectedFrameDelay - 1;
				UE_NP_TRACE_NET_RECV(ExpectedConsumeFrame, ExpectedConsumeFrame * TickState->FixedStepMS);
				UE_NP_TRACE_USER_STATE_INPUT(ModelDef, ServerRecvData.InputBuffer[Frame].Get());
			}
		}
	}

	// ------------------------------------------------------------------------------------------------------------
	// TFixedTickReplicator_Server::NetSend AP Client sending to server
	// ------------------------------------------------------------------------------------------------------------
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, FFixedTickState* TickState)
	{
		FArchive& Ar = P.Ar;
		NETSIM_CHECKSUM(Ar);

		TInstanceFrameState<ModelDef>* Frames = DataStore->Frames.Find(ID);
		npCheckSlow(Frames);

		FNetworkPredictionSerialization::WriteCompressedFrame(Ar, TickState->PendingFrame); // 1. Client's PendingFrame number

		const int32 EndFrame = TickState->PendingFrame; // PendingFrame doesn't have an input written until right before it ticks, so don't send it's contents
		const int32 StartFrame = EndFrame - NumInputsPerSend;
		
		for (int32 Frame = StartFrame; Frame < EndFrame; ++Frame)
		{
			if (Frame < 0)
			{
				EatCmd(P);
			}
			else
			{
				if (NetworkPredictionCVars::ForceSendDefaultInputCommands())
				{
					// for debugging, send blank default input instead of what we've produced locally
					static TConditionalState<InputType> DefaultInputCmd;
					FNetworkPredictionDriver<ModelDef>::NetSerialize(DefaultInputCmd, P); 
				}
				else
				{
					FNetworkPredictionDriver<ModelDef>::NetSerialize(Frames->Buffer[Frame].InputCmd, P); // 2. InputCmd
				}
			}
		}
	}

	static void EatCmd(const FNetSerializeParams& P)
	{
		TConditionalState<InputType> Empty;
		FNetworkPredictionDriver<ModelDef>::NetSerialize(Empty, P);  // 2. InputCmd	
	}

private:
	inline static int32 NumInputsPerSend = 6;
};

template<typename InModelDef>
class TIndependentTickReplicator_Server
{
public:

	using ModelDef = InModelDef;

	using StateTypes = typename ModelDef::StateTypes;
	using InputType= typename StateTypes::InputType;

	static void SetNumInputsPerSend(int32 NumToSend) 
	{ 
		NumInputsPerSend=NumToSend; 
	}

	// ------------------------------------------------------------------------------------------------------------
	// TIndependentTickReplicator_Server::NetRecv AP Client sending to server
	// ------------------------------------------------------------------------------------------------------------
	static void NetRecv(const FNetSerializeParams& P, TServerRecvData_Independent<ModelDef>& ServerRecvData, TModelDataStore<ModelDef>* DataStore)
	{
		FArchive& Ar = P.Ar;
		NETSIM_CHECKSUM(Ar);

		const int32 EndFrame = FNetworkPredictionSerialization::ReadCompressedFrame(Ar, ServerRecvData.LastRecvFrame); // 1. StartFrame
		const int32 StartFrame = EndFrame - NumInputsPerSend;

		// Reset consumed frame if we detect a gap.
		// Note this could discard unprocessed commands we previously received (but didn't process) but handling this case doesn't seem necessary or practical
		if (ServerRecvData.LastConsumedFrame+1 < StartFrame)
		{
			ServerRecvData.LastConsumedFrame = StartFrame-1;
			ServerRecvData.LastRecvFrame = StartFrame-1;
		}
		
		int32 ExpectedTimeMS = ServerRecvData.TotalSimTimeMS; // SimTime we expect to process next command at
		for (int32 Frame=ServerRecvData.LastConsumedFrame+1; Frame >= 0 && Frame <= ServerRecvData.LastRecvFrame; ++Frame)
		{
			ExpectedTimeMS += ServerRecvData.InputBuffer[Frame].DeltaTimeMS;
		}

		for (int32 Frame=StartFrame; Frame < EndFrame; ++Frame)
		{
			if (Frame <= ServerRecvData.LastRecvFrame)
			{
				EatCmd(P);
			}
			else
			{
				npEnsure(Frame >= 0);

				for (int32 DroppedFrame = ServerRecvData.LastRecvFrame+1; DroppedFrame < Frame; ++DroppedFrame)
				{
					// FIXME: trace ID has to be better
					UE_NP_TRACE_SYSTEM_FAULT("Gap in input stream detected on server. LastRecvFrame: %d. New Frame: %d", ServerRecvData.LastRecvFrame, DroppedFrame);
					ServerRecvData.InputBuffer[DroppedFrame].DeltaTimeMS = 0;
				}

				typename TServerRecvData_Independent<ModelDef>::FFrame& RecvFrame = ServerRecvData.InputBuffer[Frame];

				FNetworkPredictionDriver<ModelDef>::NetSerialize(RecvFrame.InputCmd, P); // 2. InputCmd
				FNetworkPredictionSerialization::SerializeDeltaMS(P.Ar, RecvFrame.DeltaTimeMS); // 3. DeltaTime

				// Trace what we received
				const int32 ExpectedFrameDelay = ServerRecvData.LastRecvFrame - ServerRecvData.LastConsumedFrame;
				const int32 ExpectedConsumeFrame = ServerRecvData.PendingFrame + ExpectedFrameDelay;
				
				npEnsure(ExpectedConsumeFrame >= 0);
				UE_NP_TRACE_NET_RECV(ExpectedConsumeFrame, ExpectedTimeMS);
				UE_NP_TRACE_USER_STATE_INPUT(ModelDef, ServerRecvData.InputBuffer[Frame].InputCmd.Get());

				// Advance
				ExpectedTimeMS += RecvFrame.DeltaTimeMS;
				ServerRecvData.LastRecvFrame = Frame;
			}
		}
	}

	// ------------------------------------------------------------------------------------------------------------
	// TIndependentTickReplicator_Server::NetRecv AP Client sending to server
	// ------------------------------------------------------------------------------------------------------------
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, FVariableTickState* TickState)
	{
		FArchive& Ar = P.Ar;
		NETSIM_CHECKSUM(Ar);

		TInstanceFrameState<ModelDef>* Frames = DataStore->Frames.Find(ID);
		npCheckSlow(Frames);

		FNetworkPredictionSerialization::WriteCompressedFrame(Ar, TickState->PendingFrame); // 1. Client's PendingFrame number

		const int32 EndFrame = TickState->PendingFrame; // PendingFrame doesn't have an input written until right before it ticks, so don't send it's contents
		const int32 StartFrame = EndFrame - NumInputsPerSend;

		for (int32 Frame = StartFrame; Frame < EndFrame; ++Frame)
		{
			if (Frame < 0)
			{
				EatCmd(P);
			}
			else
			{
				if (NetworkPredictionCVars::ForceSendDefaultInputCommands())
				{
					// for debugging, send blank default input instead of what we've produced locally
					static TConditionalState<InputType> DefaultInputCmd;
					FNetworkPredictionDriver<ModelDef>::NetSerialize(DefaultInputCmd, P);
				}
				else
				{
					FNetworkPredictionDriver<ModelDef>::NetSerialize(Frames->Buffer[Frame].InputCmd, P); // 2. InputCmd
				}

				FNetworkPredictionSerialization::SerializeDeltaMS(P.Ar, TickState->Frames[Frame].DeltaMS); // 3. Delta InputCmd
			}
		}
	}

private:

	static void EatCmd(const FNetSerializeParams& P)
	{
		TConditionalState<InputType> Empty;
		FNetworkPredictionDriver<ModelDef>::NetSerialize(Empty, P);  // 2. InputCmd

		int32 TimeMS = 0;
		FNetworkPredictionSerialization::SerializeDeltaMS(P.Ar, TimeMS); // 3. Delta InputCmd
	}

	inline static int32 NumInputsPerSend = 6;
};

// ---------------------------------------------------------------------------------------------------------------------------
//	Server -> AP Client
//
//	The Fixed/Independent ticking cases differ a bit but still share the same the core payload: Sync/Aux/Cues.
//	Fixed tick sends last consumed client input frame # ANd the server frame in order to correlate client/server frame numbers.
//	Independent tick sends last consumed client input frame # + TotalSimTime in order to detect dropped frames.
//
//	Where this data comes from differs between Fixed/Independent.
//
// ---------------------------------------------------------------------------------------------------------------------------

template<typename ModelDef>
class TCommonReplicator_AP
{
public:

	static void NetRecv(const FNetSerializeParams& P, TInstanceData<ModelDef>& InstanceData, TClientRecvData<ModelDef>& ClientRecvState)
	{
		NETSIM_CHECKSUM(P.Ar);

		FNetworkPredictionDriver<ModelDef>::NetSerialize(ClientRecvState.SyncState, P);	// 1. Sync
		FNetworkPredictionDriver<ModelDef>::NetSerialize(ClientRecvState.AuxState, P);	// 2. Aux
		
		UE_NP_TRACE_USER_STATE_SYNC(ModelDef, ClientRecvState.SyncState.Get());
		UE_NP_TRACE_USER_STATE_AUX(ModelDef, ClientRecvState.AuxState.Get());
	}

	static void NetSend(const FNetSerializeParams& P, TInstanceData<ModelDef>& InstanceData, typename TInstanceFrameState<ModelDef>::FFrame& FrameData)
	{
		NETSIM_CHECKSUM(P.Ar);

		FNetworkPredictionDriver<ModelDef>::NetSerialize(FrameData.SyncState, P);	// 1. Sync
		FNetworkPredictionDriver<ModelDef>::NetSerialize(FrameData.AuxState, P);	// 2. Aux
	}
};

static constexpr float TimeDilationNetSerializationScale = 10000.f;	// Used when net-serializing a float into an int16

template<typename InModelDef>
class TFixedTickReplicator_AP
{
public:

	using ModelDef = InModelDef;

	// ------------------------------------------------------------------------------------------------------------
	// AP Client receives from the server
	// ------------------------------------------------------------------------------------------------------------
	static void NetRecv(const FNetSerializeParams& P, TClientRecvData<ModelDef>& ClientRecvState, TModelDataStore<ModelDef>* DataStore, FFixedTickState* TickState)
	{
		FArchive& Ar = P.Ar;
		
		const int32 LastConsumedInputFrame = FNetworkPredictionSerialization::ReadCompressedFrame(Ar, TickState->PendingFrame); // 1. Last Consumed (Client) Input Frame
		const int32 ServerFrame = FNetworkPredictionSerialization::ReadCompressedFrame(Ar, TickState->PendingFrame + TickState->Offset); // 2. Server's Frame

		Ar.SerializeBits(&TickState->bHasTimeDilation, 1); // 3. Whether We have Time Dilation Or Not. Most Of The time We Do Not.. We Don't Need to Send int16 over network
		if (TickState->bHasTimeDilation)
		{
			int16 NetworkTimeDilation = 1;
			Ar << NetworkTimeDilation; // 4. Time Dilation
			TickState->SuggestedTimeDilation = FMath::Clamp((float)NetworkTimeDilation / TimeDilationNetSerializationScale, NetworkPredictionCVars::TimeDilationMin->GetFloat(), NetworkPredictionCVars::TimeDilationMax->GetFloat());
		}
		else
		{
			TickState->SuggestedTimeDilation = 1.f;
		}

		npEnsure(LastConsumedInputFrame <= TickState->PendingFrame);
		npEnsure(ServerFrame >= 0);

		if (LastConsumedInputFrame == INDEX_NONE)
		{
			// Server hasn't processed any of our input yet, so we don't know the offset
			// Consider: we could set a flag to indicate this and assume the mapping = tail frame
			// this could cut down on some initial corrections as the first command makes it way through
			// (mainly on initially set but no longer changing state)
			TickState->Offset = 0;
		}
		else
		{
			// Calculate TickState::Offset the difference between client and server frame numbers.
			// LocalFrame = ServerFrame - Offset

			// LastConsumedInputFrame was used as input to produce ServerFrame on the server,
			// So ServerFrame/LastConsumedInputFrame are intrinsically "one frame off".

			// We want:
			// LocalFrame + Offset = ServerFrame.

			// There for:
			// LastConsumedInputFrame + 1 + Offset = ServerFrame.
			
			TickState->Offset = ServerFrame - LastConsumedInputFrame - 1;
		}

		// AP recv drives fixed tick interpolation
		TickState->Interpolation.LatestRecvFrameAP = ServerFrame;
		TickState->ConfirmedFrame = ServerFrame - TickState->Offset;

		ClientRecvState.ServerFrame = ServerFrame;
		UE_NP_TRACE_NET_RECV(ServerFrame, ServerFrame * TickState->FixedStepMS);

		npEnsureSlow(ClientRecvState.InstanceIdx >= 0);
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvState.InstanceIdx);

		TCommonReplicator_AP<ModelDef>::NetRecv(P, InstanceData, ClientRecvState); // 5. Common

		InstanceData.CueDispatcher->NetRecvSavedCues(P.Ar, true, ServerFrame, 0); // 6. NetSimCues
	}

	// ------------------------------------------------------------------------------------------------------------
	// Server sends to AP client
	// ------------------------------------------------------------------------------------------------------------
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, const FFixedTickState* TickState)
	{
		FArchive& Ar = P.Ar;

		TInstanceData<ModelDef>* Instance = DataStore->Instances.Find(ID);
		npCheckSlow(Instance);

		TInstanceFrameState<ModelDef>* Frames = DataStore->Frames.Find(ID);
		npCheckSlow(Frames);

		int32 LastConsumedFrame = INDEX_NONE;
		int32 LastReceivedFrame = INDEX_NONE;
		TServerRecvData_Fixed<ModelDef>* ServerRecvState = DataStore->ServerRecv.Find(ID);
		if (ServerRecvState)
		{
			LastConsumedFrame = ServerRecvState->LastConsumedFrame;
			LastReceivedFrame = ServerRecvState->LastRecvFrame;
		}

		const int32 PendingFrame = TickState->PendingFrame;
		npEnsureSlow(PendingFrame >= 0);

		FNetworkPredictionSerialization::WriteCompressedFrame(Ar, LastConsumedFrame); // 1. Last Consumed Input Frame (Client's frame)
		FNetworkPredictionSerialization::WriteCompressedFrame(Ar, PendingFrame); // 2. PendingFrame (Server's frame)

		int32 BufferOffset = LastConsumedFrame - LastReceivedFrame + 2;

		if (!NetworkPredictionCVars::AllowTimeDilationEscalation->GetBool())
		{
			BufferOffset = FMath::Clamp(BufferOffset, -1, 1);
		}

		if (LastReceivedFrame == INDEX_NONE || LastConsumedFrame == INDEX_NONE)
		{
			BufferOffset = 0;
		}

		// Calculate desired dilation and send to client
		float CalculatedTimeDilation = 1.f;
		bool HasTimeDilation = false;

		if (BufferOffset != 0)
		{
			const float TimeDilationDecay = FMath::Clamp(1.0f - (NetworkPredictionCVars::TimeDilationEscalationDecay->GetFloat() * FMath::Abs(BufferOffset)), NetworkPredictionCVars::TimeDilationEscalationDecayMax->GetFloat(), 1.0f);
			CalculatedTimeDilation = 1.0f + ((NetworkPredictionCVars::TimeDilationAmount->GetFloat() * -BufferOffset) * TimeDilationDecay);
			CalculatedTimeDilation = FMath::Clamp(CalculatedTimeDilation, NetworkPredictionCVars::TimeDilationMin->GetFloat(), NetworkPredictionCVars::TimeDilationMax->GetFloat());
			HasTimeDilation = true;
		}
		else
		{
			CalculatedTimeDilation = 0.f;
		}

		Ar.SerializeBits(&HasTimeDilation, 1); // 3. Whether we have active time dilation. Most of the time we don't, so we can avoid sending the int16 over the network.
		
		if (HasTimeDilation)
		{
			int16 NetworkTimeDilation = FMath::RoundToInt(CalculatedTimeDilation * TimeDilationNetSerializationScale);
			Ar << NetworkTimeDilation; // 4. Time Dilation
		}

		TCommonReplicator_AP<ModelDef>::NetSend(P, *Instance, Frames->Buffer[PendingFrame]); // 5. Common

		Instance->CueDispatcher->NetSendSavedCues(P.Ar, ENetSimCueReplicationTarget::AutoProxy, true); // 6. NetSimCues
	}
};

template<typename InModelDef>
class TIndependentTickReplicator_AP
{
public:

	using ModelDef = InModelDef;

	// ------------------------------------------------------------------------------------------------------------
	// AP Client receives from the server
	// ------------------------------------------------------------------------------------------------------------
	static void NetRecv(const FNetSerializeParams& P, TClientRecvData<ModelDef>& ClientRecvState, TModelDataStore<ModelDef>* DataStore, FVariableTickState* TickState)
	{
		FArchive& Ar = P.Ar;
		const int32 LastConsumedInputFrame = FNetworkPredictionSerialization::ReadCompressedFrame(Ar, TickState->PendingFrame); // 1. Last Consumed (Client) Input Frame
		ClientRecvState.ServerFrame = LastConsumedInputFrame + 1;
		npEnsure(ClientRecvState.ServerFrame >= 0);

		TickState->ConfirmedFrame = ClientRecvState.ServerFrame;

		FNetworkPredictionSerialization::SerializeTimeMS(P.Ar, ClientRecvState.SimTimeMS); // 2. TotalSimTime

		UE_NP_TRACE_NET_RECV(ClientRecvState.ServerFrame, ClientRecvState.SimTimeMS);

		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvState.InstanceIdx);
		TCommonReplicator_AP<ModelDef>::NetRecv(P, InstanceData, ClientRecvState); // 3. Common
	}

	// ------------------------------------------------------------------------------------------------------------
	// Server sends to AP client
	// ------------------------------------------------------------------------------------------------------------
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, TServerRecvData_Independent<ModelDef>& ServerRecvData, const FVariableTickState* VariableTickState)
	{
		FArchive& Ar = P.Ar;
		FNetworkPredictionSerialization::WriteCompressedFrame(Ar, ServerRecvData.LastConsumedFrame); // 1. Last Consumed Input Frame (Client's frame)
		FNetworkPredictionSerialization::SerializeTimeMS(Ar, ServerRecvData.TotalSimTimeMS); // 2. TotalSimTime

		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ServerRecvData.InstanceIdx);
		TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ServerRecvData.FramesIdx);
		
		TCommonReplicator_AP<ModelDef>::NetSend(P, InstanceData, Frames.Buffer[ServerRecvData.PendingFrame]); // 3. Common
	}
};

// ---------------------------------------------------------------------------------------------------------------------------
//	Server -> SP Client
//	Like the AP case, the core payload is the same between fixed and independent: Input/Sync/Aux/Cues
//
//	There are actually 3 cases to consider here:
//	1. Fixed Tick: only sends server frame #
//	2. Independent Tick, remotely controlled: send total sim time, which comes from the server's TServerRecvData_Independent for the controlling client.
//	3. Independent Tick, locally controlled: send total sim time, which comes from the server's local VariableTickState.
//
// ---------------------------------------------------------------------------------------------------------------------------

template<typename InModelDef>
class TCommonReplicator_SP
{
public:

	using ModelDef = InModelDef;

	template<typename ClientRecvDataType>
	static void NetRecv(const FNetSerializeParams& P, ClientRecvDataType& ClientRecvState, TModelDataStore<ModelDef>* DataStore)
	{
		FArchive& Ar = P.Ar;
		NETSIM_CHECKSUM(Ar);

		FNetworkPredictionDriver<ModelDef>::NetSerialize(ClientRecvState.InputCmd, P);	// 1. Input
		FNetworkPredictionDriver<ModelDef>::NetSerialize(ClientRecvState.SyncState, P);	// 2. Sync
		FNetworkPredictionDriver<ModelDef>::NetSerialize(ClientRecvState.AuxState, P);	// 3. Aux

		UE_NP_TRACE_USER_STATE_INPUT(ModelDef, ClientRecvState.InputCmd.Get());
		UE_NP_TRACE_USER_STATE_SYNC(ModelDef, ClientRecvState.SyncState.Get());
		UE_NP_TRACE_USER_STATE_AUX(ModelDef, ClientRecvState.AuxState.Get());
	}
	
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, TInstanceData<ModelDef>* InstanceData, int32 PendingFrame)
	{
		npCheckSlow(InstanceData);

		FArchive& Ar = P.Ar;
		NETSIM_CHECKSUM(Ar);

		TInstanceFrameState<ModelDef>* Frames = DataStore->Frames.Find(ID);
		npCheckSlow(Frames);

		typename TInstanceFrameState<ModelDef>::FFrame& FrameData = Frames->Buffer[PendingFrame];
		FNetworkPredictionDriver<ModelDef>::NetSerialize(FrameData.InputCmd, P);	// 1. Input
		FNetworkPredictionDriver<ModelDef>::NetSerialize(FrameData.SyncState, P);	// 2. Sync
		FNetworkPredictionDriver<ModelDef>::NetSerialize(FrameData.AuxState, P);	// 3. Aux
	}
};

template<typename InModelDef>
class TFixedTickReplicator_SP
{
public:

	using ModelDef = InModelDef;

	// ------------------------------------------------------------------------------------------------------------
	// SP Client receives from the server
	// ------------------------------------------------------------------------------------------------------------
	static void NetRecv(const FNetSerializeParams& P, TClientRecvData<ModelDef>& ClientRecvState, TModelDataStore<ModelDef>* DataStore, FFixedTickState* TickState)
	{
		ClientRecvState.ServerFrame = FNetworkPredictionSerialization::ReadCompressedFrame(P.Ar, TickState->PendingFrame + TickState->Offset); // 1. PendingFrame (Server Frame)
		npEnsure(ClientRecvState.ServerFrame >= 0);

		TickState->Interpolation.LatestRecvFrameSP = FMath::Max(TickState->Interpolation.LatestRecvFrameSP, ClientRecvState.ServerFrame);

		UE_NP_TRACE_NET_RECV(ClientRecvState.ServerFrame, ClientRecvState.ServerFrame * TickState->FixedStepMS);
		
		TCommonReplicator_SP<ModelDef>::NetRecv(P, ClientRecvState, DataStore); // 2, Common

		npEnsureSlow(ClientRecvState.InstanceIdx >= 0);
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvState.InstanceIdx);

		const bool bSerializeCueFrames = true; // Fixed tick can use Frame numbers for SP serialization
		InstanceData.CueDispatcher->NetRecvSavedCues(P.Ar, bSerializeCueFrames, ClientRecvState.ServerFrame, 0); // 3. NetSimCues
	}

	// ------------------------------------------------------------------------------------------------------------
	// Server sends to SP Client
	// ------------------------------------------------------------------------------------------------------------
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, const FFixedTickState* TickState)
	{
		const int32 PendingFrame = TickState->PendingFrame;
		npEnsure(PendingFrame >= 0);

		TInstanceData<ModelDef>* Instance = DataStore->Instances.Find(ID);
		npCheckSlow(Instance);
		
		FNetworkPredictionSerialization::WriteCompressedFrame(P.Ar, PendingFrame); // 1. PendingFrame (Server's frame)
		
		TCommonReplicator_SP<ModelDef>::NetSend(P, ID, DataStore, Instance, PendingFrame); // 2. Common

		const bool bSerializeCueFrames = true; // Fixed tick can use Frame numbers for SP serialization
		Instance->CueDispatcher->NetSendSavedCues(P.Ar, ENetSimCueReplicationTarget::SimulatedProxy | ENetSimCueReplicationTarget::Interpolators, bSerializeCueFrames); // 3. NetSimCues
	}
};

template<typename InModelDef>
class TIndependentTickReplicator_SP
{
public:

	using ModelDef = InModelDef;

	// ------------------------------------------------------------------------------------------------------------
	// SP Client receives from the server
	// ------------------------------------------------------------------------------------------------------------
	static void NetRecv(const FNetSerializeParams& P, TClientRecvData<ModelDef>& ClientRecvState, TModelDataStore<ModelDef>* DataStore, FVariableTickState* TickState)
	{
		FNetworkPredictionSerialization::SerializeTimeMS(P.Ar, ClientRecvState.SimTimeMS); // 1. ServerTotalSimTime

		
#if UE_NP_TRACE_ENABLED
		int32 TraceSimTime = 0;
		FNetworkPredictionSerialization::SerializeTimeMS(P.Ar, TraceSimTime); // 2. ServerTotalSimTime
#else
		int32 TraceSimTime = ClientRecvState.SimTimeMS;
#endif

		// SP timestamps drive independent interpolation
		// (AP frame/time can't help here - that is the nature of independent ticking!)
		TickState->Interpolation.LatestRecvTimeMS = FMath::Max(TickState->Interpolation.LatestRecvTimeMS, ClientRecvState.SimTimeMS);
		
		// This is kinda wrong but not clear what it should be. The server's frame# is irrelevant in independent tick for SPs.
		// Should we not trace it and have insights handle this case explicitly? Or guess where it would go roughly?
		// Just tracing it as "latest" for now.
		const int32 TraceFrame = TickState->PendingFrame;
		npEnsure(TraceFrame >= 0);

		UE_NP_TRACE_NET_RECV(TraceFrame, TraceSimTime);
		
		TCommonReplicator_SP<ModelDef>::NetRecv(P, ClientRecvState, DataStore); // 3. Common

		npEnsureSlow(ClientRecvState.InstanceIdx >= 0);
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvState.InstanceIdx);

		const bool bSerializeCueFrames = true; // Fixed tick can use Frame numbers for SP serialization
		InstanceData.CueDispatcher->NetRecvSavedCues(P.Ar, bSerializeCueFrames, INDEX_NONE, ClientRecvState.SimTimeMS); // 4. NetSimCues
	}

	// ------------------------------------------------------------------------------------------------------------
	// Server sends to SP Client
	// ------------------------------------------------------------------------------------------------------------
	
	// For locally controlled/ticked actors on the server
	
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, const FVariableTickState* TickState)
	{
		const int32 TotalSimTime = TickState->Frames[TickState->PendingFrame].TotalMS;
		NetSend(P, ID, DataStore, TotalSimTime, TotalSimTime, TickState->PendingFrame);
	}

	// For remotely controlled/ticked actors on the server
	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, const TServerRecvData_Independent<ModelDef>& IndependentTickState, const FVariableTickState* VariableTickState)
	{
		// Note we are sending the (Server's) local variable tick sim time as the timestamp, not the actual independent tick.
		// Reasoning: The VariableTick timestamp is when the last tick took place on the server. Its when the stuff that appened in tick "actually happened" relative to everything else.
		// The independent tick time is really between the AP client and server. Letting this time "leak" to the SP clients means they have to deal with aligning/reconcile the timestamps
		// of the remote controlled sim differently than the non remote controlled sim. (remote controlled on the server).
		//
		// Practical reason: cues are timestamped with the variable tick time. (AP client will use frames, SP clients will use time. Easier to align the times server side than have
		// each client do it independently for each independently ticking remote controlled simulation.
		const int32 VariableTickTimeMS = VariableTickState->Frames[VariableTickState->PendingFrame].TotalMS;

		TServerRecvData_Independent<ModelDef>* IndependentTickData =  DataStore->ServerRecv_IndependentTick.Find(ID);
		npCheckSlow(IndependentTickData);
		const int32 IndependentSimTimeMS = IndependentTickData->TotalSimTimeMS;

		NetSend(P, ID, DataStore, IndependentSimTimeMS, VariableTickTimeMS, IndependentTickState.PendingFrame);
	}	

private:

	static void NetSend(const FNetSerializeParams& P, FNetworkPredictionID ID, TModelDataStore<ModelDef>* DataStore, int32 IndependentSimTime, int32 ServerTotalSimTime, int32 PendingFrame)
	{
		TInstanceData<ModelDef>* Instance = DataStore->Instances.Find(ID);
		npCheckSlow(Instance);

		FNetworkPredictionSerialization::SerializeTimeMS(P.Ar, ServerTotalSimTime); // 1. ServerTotalSimTime

#if UE_NP_TRACE_ENABLED
		FNetworkPredictionSerialization::SerializeTimeMS(P.Ar, IndependentSimTime); // 2. IndependentSimTime
#endif

		TCommonReplicator_SP<ModelDef>::NetSend(P, ID, DataStore, Instance, PendingFrame); // 3. Common

		const bool bSerializeCueFrames = false; // Independent tick cannot use Frame numbers for SP serialization (use time instead)
		Instance->CueDispatcher->NetSendSavedCues(P.Ar, ENetSimCueReplicationTarget::SimulatedProxy | ENetSimCueReplicationTarget::Interpolators, bSerializeCueFrames); // 4. NetSimCues
	}
};

// ---------------------------------------------------------------------------------------------------------------------------


#undef UE_API
