// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NetworkPredictionConfig.h"
#include "Templates/SubclassOf.h"

#include "NetworkPredictionSettings.generated.h"

class ANetworkPredictionReplicatedManager;
struct FPropertyChangedEvent;

USTRUCT(meta=(ShowOnlyInnerProperties))
struct FNetworkPredictionSettings
{
	GENERATED_BODY()

	// Which ticking policy to use in cases where both are supported by the underlying simulation.
	UPROPERTY(config, EditAnywhere, Category = Global)
	ENetworkPredictionTickingPolicy PreferredTickingPolicy = ENetworkPredictionTickingPolicy::Fixed;

	// Replicated Manager class
	UPROPERTY(config, EditAnywhere, Category = Global)
	TSubclassOf<ANetworkPredictionReplicatedManager> ReplicatedManagerClassOverride;

	// ------------------------------------------------------------------------------------------

	// Frame rate to use when running Fixed Tick simulations. Note: Engine::FixedFrameRate will take precedence if Engine::bUseFixedFrameRate is enabled.
	UPROPERTY(config, EditAnywhere, Category = FixedTick)
	int32 FixedTickFrameRate = 60;

	// Default NetworkLOD for simulated proxy simulations.
	UPROPERTY(config, EditAnywhere, Category = FixedTick)
	ENetworkLOD SimulatedProxyNetworkLOD = ENetworkLOD::ForwardPredict;

	// If enabled, the fixed tick smoothing service will be active, allowing drivers to receive smoothly interpolated updates for any locally-simulated objects (including forward-predicted sim proxies).
	UPROPERTY(config, EditAnywhere, Category = FixedTick)
	bool bEnableFixedTickSmoothing = false;

	// ------------------------------------------------------------------------------------------

	// How much buffered time to keep for fixed ticking interpolated sims (client only).
	UPROPERTY(config, EditAnywhere, Category = Interpolation)
	int32 FixedTickInterpolationBufferedMS = 100;

	// How much buffered time to keep for fixed independent interpolated sims (client only).
	UPROPERTY(config, EditAnywhere, Category = Interpolation)
	int32 IndependentTickInterpolationBufferedMS = 100;

	// Max buffered time to keep for fixed independent interpolated sims (client only).
	UPROPERTY(config, EditAnywhere, Category = Interpolation)
	int32 IndependentTickInterpolationMaxBufferedMS = 250;

	// ------------------------------------------------------------------------------------------

	// Clients should send this number of most recent input commands together per update, for a Fixed Tick simulation
	UPROPERTY(config, EditAnywhere, Category = Input)
	int32 FixedTickInputSendCount = 6;

	// Clients should send this number of most recent input commands together per update, for an Independent Tick simulation
	UPROPERTY(config, EditAnywhere, Category = Input)
	int32 IndependentTickInputSendCount = 6;

	// Cap the number of remote input commands required to be buffered before resuming input consumption after a fault
	UPROPERTY(config, EditAnywhere, Category = Input)
	int32 MaximumRemoteInputFaultLimit = 6;
};

USTRUCT(meta=(ShowOnlyInnerProperties))
struct FNetworkPredictionDevHUDItem
{
	GENERATED_BODY();

	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	FString DisplayName;

	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	FString ExecCommand;

	// Return to to level HUD menu after selecting this
	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	bool bAutoBack = true;

	// only works in PIE
	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	bool bRequirePIE = false;

	// only works in non PIE
	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	bool bRequireNotPIE = false;
};

USTRUCT(meta=(ShowOnlyInnerProperties))
struct FNetworkPredictionDevHUD
{
	GENERATED_BODY()

	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	FString HUDName;

	UPROPERTY(config, EditAnywhere, Category = DevHUD, meta=(ShowOnlyInnerProperties))
	TArray<FNetworkPredictionDevHUDItem> Items;

	// only works in PIE
	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	bool bRequirePIE = false;

	// only works in non PIE
	UPROPERTY(config, EditAnywhere, Category = DevHUD)
	bool bRequireNotPIE = false;
};


UCLASS(config=NetworkPrediction, defaultconfig, meta=(DisplayName="Network Prediction"))
class UNetworkPredictionSettingsObject : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY(config, EditAnywhere, Category = "Network Prediction", meta=(ShowOnlyInnerProperties))
	FNetworkPredictionSettings Settings;

	UPROPERTY(config, EditAnywhere, Category = DevHUD, meta=(ShowOnlyInnerProperties))
	TArray<FNetworkPredictionDevHUD> DevHUDs;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};
