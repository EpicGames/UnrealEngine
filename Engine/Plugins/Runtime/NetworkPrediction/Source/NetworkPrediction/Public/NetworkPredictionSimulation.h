// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "NetworkPredictionStateTypes.h"

template <typename ElementType> struct TNetSimLazyWriter;

struct FNetSimCueDispatcher;

// Input state is just a collection of references to the simulation state types
template<typename StateTypes=TNetworkPredictionStateTypes<>>
using TNetSimInput = TNetworkPredictionState<StateTypes>;

// Output state: the output SyncState (always created) and TNetSimLazyWriter for the AuxState (created on demand since every tick does not generate a new aux frame)
template<typename StateType=TNetworkPredictionStateTypes<>>
struct TNetSimOutput
{
	using InputType = typename StateType::InputType;
	using SyncType = typename StateType::SyncType;
	using AuxType = typename StateType::AuxType;

	SyncType* Sync;
	const TNetSimLazyWriter<AuxType>& Aux;
	FNetSimCueDispatcher& CueDispatch;

	TNetSimOutput(SyncType* InSync, const TNetSimLazyWriter<AuxType>& InAux, FNetSimCueDispatcher& InCueDispatch)
		: Sync(InSync), Aux(InAux), CueDispatch(InCueDispatch) { }
};
