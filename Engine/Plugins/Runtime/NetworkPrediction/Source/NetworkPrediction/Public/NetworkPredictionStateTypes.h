// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HAL/Platform.h"

// Enum to identify the state types
enum class ENetworkPredictionStateType : uint8
{
	Input,
	Sync,
	Aux
};

inline const TCHAR* LexToString(ENetworkPredictionStateType A)
{
	switch(A)
	{
		case ENetworkPredictionStateType::Input: return TEXT("Input");
		case ENetworkPredictionStateType::Sync: return TEXT("Sync");
		case ENetworkPredictionStateType::Aux: return TEXT("Aux");
	};
	return TEXT("Unknown");
}

// State type defines
template<typename InInputCmd=void, typename InSyncState=void, typename InAuxState=void>
struct TNetworkPredictionStateTypes
{
	using InputType = InInputCmd;
	using SyncType = InSyncState;
	using AuxType = InAuxState;
};

// Tuple of state types
template<typename StateType=TNetworkPredictionStateTypes<>>
struct TNetworkPredictionState
{
	using InputType = typename StateType::InputType;
	using SyncType = typename StateType::SyncType;
	using AuxType = typename StateType::AuxType;

	const InputType* Cmd;
	const SyncType* Sync;
	const AuxType* Aux;

	TNetworkPredictionState(const InputType* InInputCmd, const SyncType* InSync, const AuxType* InAux)
		: Cmd(InInputCmd), Sync(InSync), Aux(InAux) { }	

	// Allows implicit downcasting to a parent simulation's types
	template<typename T>
	TNetworkPredictionState(const TNetworkPredictionState<T>& Other)
		: Cmd(Other.Cmd), Sync(Other.Sync), Aux(Other.Aux) { }
};

// Just the Sync/Aux pair
template<typename StateType=TNetworkPredictionStateTypes<>>
struct TSyncAuxPair
{
	using SyncType = typename StateType::SyncType;
	using AuxType = typename StateType::AuxType;

	const SyncType* Sync;
	const AuxType* Aux;

	TSyncAuxPair(const SyncType* InSync, const AuxType* InAux)
		: Sync(InSync), Aux(InAux) { }	

	// Allows implicit downcasting to a parent simulation's types
	template<typename T>
	TSyncAuxPair(const TSyncAuxPair<T>& Other)
		: Sync(Other.Sync), Aux(Other.Aux) { }
};
