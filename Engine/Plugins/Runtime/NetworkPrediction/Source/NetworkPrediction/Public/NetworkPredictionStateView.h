// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

struct FNetSimCueDispatcher;

// Generic view into a managed instance's state
struct FNetworkPredictionStateView
{
	// Simulation PendingFrame number. This is the "server frame" number that will be used as input for the next tick.
	// This can be used for server authoritative timers/countdowns etc but should not be used to index into local frame
	// buffer storage. Local frame numbers are stored on the NetworkPredictionWorldManager's internal tick states.
	int32 PendingFrame = 0;

	// Latest Simulation time. Like PendingFrame this is the "server time" and is synchronized.
	// This is redundant in fixed tick scenerios but is needed for independent ticking
	// (Consider: maybe this is rarely used enough that a more expensive lookup would be better than cacheing it in the view?)
	int32 SimTimeMS = 0;

	// ::SimulationTick is in progress
	bool bTickInProgress = false;
		
	// Pending states: these are what will be used as input into the next SimulationTick call, if we are running a local tick.
	// if there is no local tick, for example in interpolation mode, these will set to the latest consume simulation frame.
	// (so, latest simulation frame used in interpolation for example. But not necessarily the latest received frame)
	void* PendingInputCmd = nullptr;
	void* PendingSyncState = nullptr;
	void* PendingAuxState = nullptr;

	// Presentation states: the latest locally smoothed/interpolated states that will not be fed back into the sim 
	// (these will be null in cases where there is no smoothing/interpolation)
	void* PresentationSyncState = nullptr;
	void* PresentationAuxState = nullptr;

	// Previous presentation states: these may be used by a smoothing service to represent state that we're smoothing away from.
	// These will not be fed back into the simulation, and they will be null in cases where there is no smoothing.
	void* PrevPresentationSyncState = nullptr;
	void* PrevPresentationAuxState = nullptr;

	// CueDispatcher is exposed so that game code can invoke OOB cues
	// Future versions may move this / make Invoking OOB cues go through a different API
	FNetSimCueDispatcher* CueDispatcher = nullptr;

	void UpdateView(int32 Frame, int32 InSimTimMS, void* Input, void* Sync, void* Aux)
	{
		PendingFrame = Frame;
		SimTimeMS = InSimTimMS;

		PendingInputCmd = Input;
		PendingSyncState = Sync;
		PendingAuxState = Aux;
	}

	void UpdatePresentationView(void* Sync, void* Aux)
	{
		PresentationSyncState = Sync;
		PresentationAuxState = Aux;
	}

	void UpdatePrevPresentationView(void* Sync, void* Aux)
	{
		PrevPresentationSyncState = Sync;
		PrevPresentationAuxState = Aux;
	}
	
	void ClearPresentationView()
	{
		PresentationSyncState = nullptr;
		PresentationAuxState = nullptr;
		PrevPresentationSyncState = nullptr;
		PrevPresentationAuxState = nullptr;
	}
};