// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "NetworkPredictionBuffer.h"

namespace Chaos { class FRewindData; }

// TimeStep info that is passed into SimulationTick
struct FNetSimTimeStep
{
	// The delta time step for this tick
	int32 StepMS;

	// How much simulation time has ran up until this point. This is "Server time", e.g, everyone agrees on this time and it can
	// be used for timers/cooldowns etc in the simulation code. It can be stored in sync/aux state and reconciles pred vs authority.
	// This will be 0 the first time ::SimulationTick runs (globally for fix tick, local tick and per-sim for remote independent sims)
	int32 TotalSimulationTime;

	// The Simulation Frame number we are computing in this tick, E.g, the output frame.
	// This is the global, everyone agrees-upon frame number. E.g, the "server frame" number.	
	// This will be 1 the first time ::SimulationTick runs. (0 is the starting input and is not generated in a Tick)
	// (globally for fix tick, local tick and per-sim for remote independent sims)
	int32 Frame;
};

// Data that is needed to tick the internal ticking services but is not passed to the user code
struct FServiceTimeStep
{
	// The local frame number, this is what should be used when mapping to local frame buffers for storage
	int32 LocalInputFrame;
	int32 LocalOutputFrame;

	// Ending total sim time, needed for Cue dispatching
	int32 EndTotalSimulationTime;
};

// ---------------------------------------------------------------------------
// (Global) Tick state for fixed tick services
//
// Notes about Fix Ticking in NetworkPrediction:
//	1. FixedTick mode will accumulate real time and run 0-N Sim frames per engine frame.
//
// ---------------------------------------------------------------------------
struct FFixedTickState
{
	// FixedStepMS that simulations should use
	int32 FixedStepMS = 33;

	// Realtime steps. That is, one of these = one FixedStepMS in simulation time.
	// This means sim time ticks slightly slower than real time.
	float FixedStepRealTimeMS = 1.0f /30.0f;

	// If time dilation is enabled, one of these = one FixedStepMS in simulation time. This will shrink or grow slightly as network latency changes.
	float FixedStepDilatedRealTimeMS = 1.0f / 30.0f;

	// Next frame to be ticked (used as input to generated PendingFrame+1)
	int32 PendingFrame=0;

	// Latest confirmed local frame number. Anything at or before this frame is "set in stone"
	int32 ConfirmedFrame=INDEX_NONE;
	
	// Maps ForwardPredicted authority frames to local frame.
	// E.g, server says "I processed your frame 1 on my frame 101" client calcs Offset as 100.
	// "LocalFrame" = "ServerFrame" - Offset. 
	int32 Offset=0;

	// Accumulates raw delta time into our fixed steps
	float UnspentTimeMS = 0.f;

	// Signals whether any time dilation is active. If so, use @SuggestedTimeDilation
	bool bHasTimeDilation = false;
	
	// Only valid when bHasTimeDilation is true. Represents a percentage of time speed up/down.  1.0 = no dilation.
	float SuggestedTimeDilation = 1.f;

	struct FInterpolationState
	{
		float AccumulatedTimeMS = 0.f; // accumulated real time
		int32 LatestRecvFrameAP = INDEX_NONE;	// Latest server frame we received, set by the AP
		int32 LatestRecvFrameSP = INDEX_NONE;	// Latest server frame we received, set by the SP
		int32 ToFrame = INDEX_NONE;	// Where we are interpolating to (ToFrame-1 -> ToFrame. Both should be valid at all times for anyone interpolating)
		float PCT = 0.f;

		int32 InterpolatedTimeMS = 0;
	};

	FInterpolationState Interpolation;

	FNetSimTimeStep GetNextTimeStep() const
	{
		return FNetSimTimeStep{FixedStepMS, GetTotalSimTimeMS(),  PendingFrame+1 + Offset};
	}

	FServiceTimeStep GetNextServiceTimeStep() const
	{
		return FServiceTimeStep{PendingFrame, PendingFrame+1,(PendingFrame+Offset+1) * FixedStepMS};
	}

	int32 GetTotalSimTimeMS() const
	{
		return (PendingFrame+Offset) * FixedStepMS; 
	}
};

// Variable tick state tracking for independent tick services
struct FVariableTickState
{
	struct FFrame
	{
		int32 DeltaMS=0;
		int32 TotalMS=0;
	};

	TNetworkPredictionBuffer<FFrame> Frames;

	FVariableTickState() : Frames(64) { }

	int32 PendingFrame = 0;
	int32 ConfirmedFrame = INDEX_NONE;

	float UnspentTimeMS = 0.f;

	struct FInterpolationState
	{
		float fTimeMS = 0.f;
		int32 LatestRecvTimeMS = 0;
	};

	FInterpolationState Interpolation;

	FNetSimTimeStep GetNextTimeStep() const
	{
		return GetNextTimeStep(Frames[PendingFrame]);
	}

	FNetSimTimeStep GetNextTimeStep(const FFrame& PendingFrameData) const
	{
		return FNetSimTimeStep{PendingFrameData.DeltaMS, PendingFrameData.TotalMS, PendingFrame+1};
	}

	FServiceTimeStep GetNextServiceTimeStep(const FFrame& PendingFrameData) const
	{
		return FServiceTimeStep{PendingFrame, PendingFrame+1, PendingFrameData.TotalMS + PendingFrameData.DeltaMS};
	}
};
