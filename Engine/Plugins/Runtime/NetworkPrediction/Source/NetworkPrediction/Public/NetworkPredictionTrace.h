// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionDriver.h"
#include "NetworkPredictionID.h"

#define UE_API NETWORKPREDICTION_API

enum ENetRole : int;
struct FNetworkPredictionInstanceArchetype;
struct FNetworkPredictionInstanceConfig;

#ifndef UE_NP_TRACE_ENABLED
#define UE_NP_TRACE_ENABLED !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#endif

// Tracing user state (content) can generate a lot of data. So this can be turned off here
#ifndef UE_NP_TRACE_USER_STATES_ENABLED
#define UE_NP_TRACE_USER_STATES_ENABLED 1
#endif


#if UE_NP_TRACE_ENABLED

// Traces what caused a reconcile (added in user ShouldReconcile function)
// Note this actually adds the 'return true' logic for reconciliation
#define UE_NP_TRACE_RECONCILE(Condition, Str) if (Condition) { FNetworkPredictionTrace::TraceReconcile(Str); return true; }

// General trace to push the active simulation's trace ID
#define UE_NP_TRACE_SIM(TraceID) FNetworkPredictionTrace::TraceSimulationScope(TraceID)

// Called when simulation is created. (Note this also sets a Scope for tracing the initial user states next)
#define UE_NP_TRACE_SIM_CREATED(ID, Driver, ModelDef) FNetworkPredictionTrace::TraceSimulationCreated<ModelDef>(ID, Driver)

// Trace config of sim changing
#define UE_NP_TRACE_SIM_CONFIG(TraceID, NetRole, bHasNetConnection, Archetype, Config, ServiceMask) FNetworkPredictionTrace::TraceSimulationConfig(TraceID, NetRole, bHasNetConnection, Archetype, Config, (int32)ServiceMask);

// Called when a PIE session is started. This is so we can keep our sets of worlds/simulations separate in between runs.
#define UE_NP_TRACE_PIE_START() FNetworkPredictionTrace::TracePIEStart()

// Called during WorldPreInit. This mainly just ensure we have a valid trace context so that actors loaded with the map can trace their \initialization
#define UE_NP_TRACE_WORLD_PREINIT() FNetworkPredictionTrace::TraceWorldPreInit()

// Generic fault/error message that gets bubbled up in the NP Insights UI
#define UE_NP_TRACE_SYSTEM_FAULT(Format, ...) FNetworkPredictionTrace::TraceSystemFault(TEXT(Format), ##__VA_ARGS__)

// Trace engine frame starting for GameInstance
#define UE_NP_TRACE_WORLD_FRAME_START(GameInstance, DeltaSeconds) FNetworkPredictionTrace::TraceWorldFrameStart(GameInstance, DeltaSeconds)

// Called to set the general tick state
#define UE_NP_TRACE_PUSH_TICK(StartMS, DeltaMS, OutputFrame) FNetworkPredictionTrace::TraceTick(StartMS, DeltaMS, OutputFrame)

// Called when an actual instance ticks (after calling UE_NP_TRACE_PUSH_TICK)
#define UE_NP_TRACE_SIM_TICK(TraceID) FNetworkPredictionTrace::TraceSimTick(TraceID)

// General trace to push the active simulation's trace ID and setup for updating the traced state for an already ticked frame (Used to update async data after completion)
#define UE_NP_TRACE_SIM_STATE(TraceID) FNetworkPredictionTrace::TraceSimState(TraceID)

// Called when we receive networked data (regardless of what we end up doing with it)
#define UE_NP_TRACE_NET_RECV(Frame, TimeMS) FNetworkPredictionTrace::TraceNetRecv(Frame, TimeMS)

// Called when ShouldReconcile returns true, signaling a rollback/correction is required
#define UE_NP_TRACE_SHOULD_RECONCILE(TraceID) FNetworkPredictionTrace::TraceShouldReconcile(TraceID)

// Called when received data is injected back into the local frame buffer (Note that the sim itself may not have been in error, we may be rolling "everything" back)
#define UE_NP_TRACE_ROLLBACK_INJECT(TraceID) FNetworkPredictionTrace::TraceRollbackInject(TraceID)

// Called before running input producing services
#define UE_NP_TRACE_PUSH_INPUT_FRAME(Frame) FNetworkPredictionTrace::TracePushInputFrame(Frame)

// Traces current local frame # -> server frame # offset
#define UE_NP_TRACE_FIXED_TICK_OFFSET(Offset, bChanged) FNetworkPredictionTrace::TraceFixedTickOffset(Offset, bChanged)

// Called to trace buffered input state
#define UE_NP_TRACE_BUFFERED_INPUT(NumBufferedFrames, bFault) FNetworkPredictionTrace::TraceBufferedInput(NumBufferedFrames, bFault)

// Trace call to Driver's ProduceInput function
#define UE_NP_TRACE_PRODUCE_INPUT(TraceID) FNetworkPredictionTrace::TraceProduceInput(TraceID)

// Called to indicate we are about to write state to the buffers outside of the normal simulation tick/netrecive. TODO: add char* identifier to debug where the mod came from
#define UE_NP_TRACE_OOB_STATE_MOD(TraceID, Frame, StrView) FNetworkPredictionTrace::TraceOOBStateMod(TraceID, Frame, StrView)

// Called whenever a new user state has been inserted into the buffers. Analysis will determine "how" it got there from previous trace events
#define UE_NP_TRACE_USER_STATE_INPUT(ModelDef, UserState) FNetworkPredictionTrace::TraceUserState<ModelDef>(UserState, FNetworkPredictionTrace::ETraceUserState::Input)
#define UE_NP_TRACE_USER_STATE_SYNC(ModelDef, UserState) FNetworkPredictionTrace::TraceUserState<ModelDef>(UserState, FNetworkPredictionTrace::ETraceUserState::Sync)
#define UE_NP_TRACE_USER_STATE_AUX(ModelDef, UserState) FNetworkPredictionTrace::TraceUserState<ModelDef>(UserState, FNetworkPredictionTrace::ETraceUserState::Aux)

#else

#include "NetworkPredictionLog.h"

// Compiled out
#define UE_NP_TRACE_RECONCILE(Condition, Str) if (Condition) { return true; }
#define UE_NP_TRACE_SIM(...)
#define UE_NP_TRACE_SIM_CREATED(...)
#define UE_NP_TRACE_SIM_CONFIG(...)

#define UE_NP_TRACE_PIE_START(...)
#define UE_NP_TRACE_WORLD_PREINIT(...)
#define UE_NP_TRACE_SYSTEM_FAULT(Format, ...) UE_LOG(LogNetworkPrediction, Warning, TEXT(Format), ##__VA_ARGS__);
#define UE_NP_TRACE_WORLD_FRAME_START(...)
#define UE_NP_TRACE_PUSH_TICK(...)
#define UE_NP_TRACE_SIM_TICK(...)
#define UE_NP_TRACE_SIM_STATE(...)
#define UE_NP_TRACE_NET_RECV(...)
#define UE_NP_TRACE_SHOULD_RECONCILE(...)
#define UE_NP_TRACE_ROLLBACK_INJECT(...)
#define UE_NP_TRACE_PUSH_INPUT_FRAME(...)
#define UE_NP_TRACE_FIXED_TICK_OFFSET(...)
#define UE_NP_TRACE_PRODUCE_INPUT(...)
#define UE_NP_TRACE_BUFFERED_INPUT(...)
#define UE_NP_TRACE_OOB_STATE_MOD(...)

#define UE_NP_TRACE_USER_STATE_INPUT(...)
#define UE_NP_TRACE_USER_STATE_SYNC(...)
#define UE_NP_TRACE_USER_STATE_AUX(...)

#endif // UE_NP_TRACE_ENABLED

UE_TRACE_CHANNEL_EXTERN(NetworkPredictionChannel, NETWORKPREDICTION_API);

class AActor;
class UGameInstance;

template<typename Model>
struct TNetworkedSimulationState;

class FNetworkPredictionTrace
{
public:

	template<typename ModelDef>
	static void TraceSimulationCreated(FNetworkPredictionID ID, typename ModelDef::Driver* Driver)
	{
		TStringBuilder<256> Builder;
		FNetworkPredictionDriver<ModelDef>::GetTraceString(Driver, Builder);
		TraceSimulationCreated_Internal(ID, Builder);
	}

	static UE_API void TraceWorldFrameStart(UGameInstance* GameInstance, float DeltaSeconds);
	static UE_API void TraceSimulationConfig(int32 TraceID, ENetRole NetRole, bool bHasNetConnection, const FNetworkPredictionInstanceArchetype& Archetype, const FNetworkPredictionInstanceConfig& Config, int32 ServiceMask);

	static UE_API void TracePIEStart();
	static UE_API void TraceWorldPreInit();
	static UE_API void TraceSystemFault(const TCHAR* Fmt, ...);

	static UE_API void TraceSimulationScope(int32 TraceID);

	static UE_API void TraceTick(int32 StartMS, int32 DeltaMS, int32 OutputFrame);
	static UE_API void TraceSimTick(int32 TraceID);
	// Used to lazy update user state after a simtick
	static UE_API void TraceSimState(int32 TraceID);

	static UE_API void TraceNetRecv(int32 Frame, int32 TimeMS);
	
	static UE_API void TraceReconcile(const FAnsiStringView& StrView); 

	static UE_API void TraceShouldReconcile(int32 TraceID);
	static UE_API void TraceRollbackInject(int32 TraceID);

	static UE_API void TracePushInputFrame(int32 Frame);
	static UE_API void TraceFixedTickOffset(int32 Offset, bool bChanged);
	static UE_API void TraceBufferedInput(int32 NumBufferedFrames, bool bFault);
	static UE_API void TraceProduceInput(int32 TraceID);

	static UE_API void TraceOOBStateMod(int32 SimulationId, int32 Frame, const FAnsiStringView& StrView);

	enum class ETraceUserState : uint8
	{
		Input,
		Sync,
		Aux,
	};

	template<typename ModelDef, typename StateType>
	static void TraceUserState(const StateType* State, ETraceUserState StateTypeEnum)
	{
#if UE_NP_TRACE_USER_STATES_ENABLED
		if (std::is_void_v<StateType>)
		{
			return;
		}

		if (UE_TRACE_CHANNELEXPR_IS_ENABLED(NetworkPredictionChannel))
		{
			npCheckSlow(State);

			TAnsiStringBuilder<512> Builder;
			FNetworkPredictionDriver<ModelDef>::TraceUserStateString(State, Builder);
			TraceUserState_Internal(StateTypeEnum, Builder);
		}
#endif
	}

private:

	static UE_API void TraceSimulationCreated_Internal(FNetworkPredictionID ID, FStringBuilderBase& Builder);

	// ----------------------------------------------------------------------------------
	

	static UE_API void TraceUserState_Internal(ETraceUserState StateType, FAnsiStringBuilderBase& Builder);

	friend struct FScopedSimulationTraceHelper;
};

#undef UE_API
