// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionCheck.h"
#include "UObject/NameTypes.h"

namespace UE_NP
{
#ifndef NP_MAX_ASYNC_MODEL_DEFS
#define NP_MAX_ASYNC_MODEL_DEFS 16
#endif

	const int32 MaxAsyncModelDefs = NP_MAX_ASYNC_MODEL_DEFS;

#ifndef NP_NUM_FRAME_STORAGE
#define NP_NUM_FRAME_STORAGE 64
#endif

	const int32 NumFramesStorage = NP_NUM_FRAME_STORAGE;

#ifndef NP_FRAME_STORAGE_GROWTH
#define NP_FRAME_STORAGE_GROWTH 8
#endif

	const int32 FrameStorageGrowth = NP_FRAME_STORAGE_GROWTH;

#ifndef NP_FRAME_INPUTCMD_BUFFER_SIZE
#define NP_FRAME_INPUTCMD_BUFFER_SIZE 16
#endif

	const int32 InputCmdBufferSize = NP_FRAME_INPUTCMD_BUFFER_SIZE;

#ifndef NP_INLINE_SIMOBJ_INPUTS
#define NP_INLINE_SIMOBJ_INPUTS 3
#endif

	const int32 InlineSimObjInputs = NP_INLINE_SIMOBJ_INPUTS;

};

// Sets index to value, resizing bit array if necessary and setting new bits to false
template<typename BitArrayType>
void NpResizeAndSetBit(BitArrayType& BitArray, int32 Index, bool Value=true)
{
	if (!BitArray.IsValidIndex(Index))
	{
		const int32 PreNum = BitArray.Num();
		BitArray.SetNumUninitialized(Index+1);
		BitArray.SetRange(PreNum, BitArray.Num() - PreNum, false);
		npCheckSlow(BitArray.IsValidIndex(Index));
	}

	BitArray[Index] = Value;
}

// Resize BitArray to NewNum, setting default value of new bits to false
template<typename BitArrayType>
void NpResizeBitArray(BitArrayType& BitArray, int32 NewNum)
{
	if (BitArray.Num() < NewNum)
	{
		const int32 PreNum = BitArray.Num();
		BitArray.SetNumUninitialized(NewNum);
		BitArray.SetRange(PreNum, BitArray.Num() - PreNum, false);
		npCheckSlow(BitArray.Num() == NewNum);
	}
}

// Set bit array contents to false
template<typename BitArrayType>
void NpClearBitArray(BitArrayType& BitArray)
{
	BitArray.SetRange(0, BitArray.Num(), false);
}

template<typename ArrayType>
void NpResizeForIndex(ArrayType& Array, int32 Index)
{
	npEnsure(Index >= 0);
	if (Array.IsValidIndex(Index) == false)
	{
		Array.SetNum(Index + UE_NP::FrameStorageGrowth);
	}
}

