// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Subsystems/WorldSubsystem.h"

#include "Services/NetworkPredictionServiceRegistry.h"
#include "NetworkPredictionSerialization.h"

#include "NetworkPredictionWorldManager.generated.h"

#define UE_API NETWORKPREDICTION_API

class FChaosSolversModule;
class ANetworkPredictionReplicatedManager;

namespace UE::Net::Private
{
	struct FNetPredictionTestWorld;
}

UCLASS(MinimalAPI)
class UNetworkPredictionWorldManager : public UWorldSubsystem
{
	GENERATED_BODY()
public:

	static UE_API UNetworkPredictionWorldManager* ActiveInstance;

	UE_API UNetworkPredictionWorldManager();

	// Server created, replicated manager (only used for centralized/system wide data replication)
	UPROPERTY()
	TObjectPtr<ANetworkPredictionReplicatedManager> ReplicatedManager = nullptr;

	// Subsystem Init/Deinit
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	// Returns unique ID for a new simulation instance regardless of type.
	// Clients set bForClient to get a negative, temp ID to use until server version comes in
	FNetworkPredictionID CreateSimulationID(const bool bForClient)
	{
		return FNetworkPredictionID(bForClient ? TempClientSpawnCount-- : InstanceSpawnCounter++);
	}

	template<typename ModelDef>
	void RemapClientSimulationID(FNetworkPredictionID ClientID, FNetworkPredictionID ServerID)
	{
		npEnsure((int32)ClientID < INDEX_NONE && (int32)ServerID >= 0);
		npEnsure(ClientID.GetTraceID() == ServerID.GetTraceID());
		
		TModelDataStore<ModelDef>* DataStore = Services.GetDataStore<ModelDef>();
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.FindOrAdd(ServerID);
		InstanceData = MoveTemp(DataStore->Instances.FindOrAdd(ClientID));
		DataStore->Instances.Remove(ClientID);
	}

	template<typename ModelDef>
	void RegisterInstance(FNetworkPredictionID ID, const TNetworkPredictionModelInfo<ModelDef>& ModelInfo)
	{
		// Register with data store
		TModelDataStore<ModelDef>* DataStore = Services.GetDataStore<ModelDef>();
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.FindOrAdd(ID);
		InstanceData.Info = ModelInfo;
		InstanceData.TraceID = ID.GetTraceID();
		InstanceData.CueDispatcher->Driver = ModelInfo.Driver; // Awkward: we should convert Cues to a service so this isn't needed.
		InstanceData.Info.View->CueDispatcher = &InstanceData.CueDispatcher.Get(); // Double awkward: we should move Cuedispatcher and clean up these weird links
	}

	template<typename ModelDef>
	void UnregisterInstance(FNetworkPredictionID ID)
	{
		if (!bLockServices)
		{
			Services.UnregisterInstance<ModelDef>(ID);
		}
		else
		{
			DeferredServiceConfigDelegate.AddLambda([ID](UNetworkPredictionWorldManager* Manager)
			{
				Manager->Services.UnregisterInstance<ModelDef>(ID);
			});
		}
	}

	template<typename ModelDef>
	void ConfigureInstance(FNetworkPredictionID ID, const FNetworkPredictionInstanceArchetype& Archetype, const FNetworkPredictionInstanceConfig& Config, FReplicationProxySet RepProxies, ENetRole Role, bool bHasNetConnection);
		
	UE_API ENetworkPredictionTickingPolicy PreferredDefaultTickingPolicy() const;

	UE_API void SyncNetworkPredictionSettings(const UNetworkPredictionSettingsObject* Settings);

	const FNetworkPredictionSettings& GetSettings() const { return Settings; }

	const FFixedTickState& GetFixedTickState() const { return FixedTickState; }
	const FVariableTickState& GetVariableTickState() const { return VariableTickState; }
	
private:

	FNetworkPredictionSettings Settings;

	FFixedTickState FixedTickState;
	FVariableTickState VariableTickState;
	FNetworkPredictionServiceRegistry Services;

	UE_API void OnWorldPreTick(UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds);
	UE_API void ReconcileSimulationsPostNetworkUpdate();
	UE_API void BeginNewSimulationFrame(UWorld* InWorld, ELevelTick InLevelTick, float InDeltaSeconds);

	// For ease of unit testing
	friend struct UE::Net::Private::FNetPredictionTestWorld;
	UE_API void OnWorldPreTick_Internal(float InDeltaSeconds, float InFixedFrameRate);
	UE_API void ReconcileSimulationsPostNetworkUpdate_Internal();
	UE_API void BeginNewSimulationFrame_Internal(float InDeltaSeconds);

	FDelegateHandle PreTickDispatchHandle;
	FDelegateHandle PostTickDispatchHandle;
	FDelegateHandle PreWorldActorTickHandle;

	int32 InstanceSpawnCounter = 1;
	int32 TempClientSpawnCount = -2; // negative IDs for client to use before getting server assigned id
	int32 LastFixedTickOffset = 0;

	// Callbacks to change subscribed services, that couldn't be made inline
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnHandleDeferredServiceConfig, UNetworkPredictionWorldManager*);
	FOnHandleDeferredServiceConfig DeferredServiceConfigDelegate;
	bool bLockServices = false; // Services are locked and we can't

	// ---------------------------------------

	template<typename ModelDef>
	void BindServerNetRecv_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore);

	template<typename ModelDef>
	void BindServerNetRecv_Independent(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore);

	// ----

	template<typename ReplicatorType, typename ModelDef=typename ReplicatorType::ModelDef>
	void BindClientNetRecv_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole);

	template<typename ReplicatorType, typename ModelDef=typename ReplicatorType::ModelDef>
	void BindClientNetRecv_Independent(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole);

	// ----

	template<typename ReplicatorType, typename ModelDef=typename ReplicatorType::ModelDef>
	void BindNetSend_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore);

	template<typename ReplicatorType, typename ModelDef=typename ReplicatorType::ModelDef>
	void BindNetSend_IndependentLocal(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore);

	template<typename ReplicatorType, typename ModelDef=typename ReplicatorType::ModelDef>
	void BindNetSend_IndependentRemote(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore);

	// ----
	
	template<typename ReplicatorType, typename ModelDef=typename ReplicatorType::ModelDef>
	void BindReplayNetSendRecv_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole);

	template<typename ReplicatorType, typename ModelDef = typename ReplicatorType::ModelDef>
	void BindReplayNetSendRecv_IndependentLocal(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole);

	template<typename ReplicatorType, typename ModelDef = typename ReplicatorType::ModelDef>
	void BindReplayNetSendRecv_IndependentRemote(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole);

	// ---------------------------------------

	template<typename ModelDef>
	void InitClientRecvData(FNetworkPredictionID ID, TClientRecvData<ModelDef>& ClientRecvData, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole);
};


// This is the function that takes the config and current network state (role/connection) and subscribes to 
// the appropriate internal NetworkPrediction services.
template<typename ModelDef>
void UNetworkPredictionWorldManager::ConfigureInstance(FNetworkPredictionID ID, const FNetworkPredictionInstanceArchetype& Archetype, const FNetworkPredictionInstanceConfig& Config, FReplicationProxySet RepProxies, ENetRole Role, bool bHasNetConnection)
{
	static constexpr FNetworkPredictionModelDefCapabilities Capabilities = FNetworkPredictionDriver<ModelDef>::GetCapabilities();

	npCheckSlow((int32)ID > 0);
	npEnsure(Role != ROLE_None);

	TModelDataStore<ModelDef>* DataStore = Services.GetDataStore<ModelDef>();
	TInstanceData<ModelDef>& InstanceData = *DataStore->Instances.Find(ID);
	TInstanceFrameState<ModelDef>& FrameData = DataStore->Frames.FindOrAdd(ID);

	const int32 PrevPendingFrame = InstanceData.Info.View->PendingFrame;
	const bool bNewInstance = (InstanceData.NetRole == ROLE_None);
	InstanceData.NetRole = Role;

	TRemoteInputService<ModelDef>::SetMaxFaultLimit(Settings.MaximumRemoteInputFaultLimit);

	ENetworkPredictionService ServiceMask = ENetworkPredictionService::None;
	
	if (Archetype.TickingMode == ENetworkPredictionTickingPolicy::Independent)
	{
		// Point cached view to the VariableTickState's pending frame
		InstanceData.Info.View->UpdateView(VariableTickState.PendingFrame, 
			VariableTickState.GetNextTimeStep().TotalSimulationTime,
			&FrameData.Buffer[VariableTickState.PendingFrame].InputCmd, 
			&FrameData.Buffer[VariableTickState.PendingFrame].SyncState, 
			&FrameData.Buffer[VariableTickState.PendingFrame].AuxState);

		switch (Role)
		{
			case ENetRole::ROLE_Authority:
			{
				if (bHasNetConnection)
				{
					// Remotely controlled
					BindServerNetRecv_Independent<ModelDef>(ID, RepProxies.ServerRPC, DataStore);
					BindNetSend_IndependentRemote<TIndependentTickReplicator_AP<ModelDef>>(ID, RepProxies.AutonomousProxy, DataStore);
					BindNetSend_IndependentRemote<TIndependentTickReplicator_SP<ModelDef>>(ID, RepProxies.SimulatedProxy, DataStore);
					BindReplayNetSendRecv_IndependentRemote<TIndependentTickReplicator_SP<ModelDef>>(ID, RepProxies.Replay, DataStore, Role);

					ServiceMask |= ENetworkPredictionService::IndependentRemoteTick;
					ServiceMask |= ENetworkPredictionService::IndependentRemoteFinalize;

					// Point view to the ServerRecv PendingFrame instead
					TServerRecvData_Independent<ModelDef>* ServerRecvData = DataStore->ServerRecv_IndependentTick.Find(ID);
					npCheckSlow(ServerRecvData);

					const int32 ServerRecvPendingFrame = ServerRecvData->PendingFrame;

					auto& PendingFrameData = FrameData.Buffer[ServerRecvPendingFrame];
					InstanceData.Info.View->UpdateView(ServerRecvPendingFrame,
						ServerRecvData->TotalSimTimeMS,
						&PendingFrameData.InputCmd, 
						&PendingFrameData.SyncState, 
						&PendingFrameData.AuxState);
				}
				else
				{
					// Locally controlled
					BindNetSend_IndependentLocal<TIndependentTickReplicator_SP<ModelDef>>(ID, RepProxies.SimulatedProxy, DataStore);
					BindReplayNetSendRecv_IndependentLocal<TIndependentTickReplicator_SP<ModelDef>>(ID, RepProxies.Replay, DataStore, Role);

					if (FNetworkPredictionDriver<ModelDef>::HasSimulation())
					{
						if (FNetworkPredictionDriver<ModelDef>::HasInput())
						{
							ServiceMask |= ENetworkPredictionService::IndependentLocalInput;
						}
						ServiceMask |= ENetworkPredictionService::IndependentLocalTick;
						ServiceMask |= ENetworkPredictionService::IndependentLocalFinalize;
					}
				}

				break;
			}
			case ENetRole::ROLE_AutonomousProxy:
			{
				BindClientNetRecv_Independent<TIndependentTickReplicator_AP<ModelDef>>(ID, RepProxies.AutonomousProxy, DataStore, Role);
				BindNetSend_IndependentLocal<TIndependentTickReplicator_Server<ModelDef>>(ID, RepProxies.ServerRPC, DataStore);
				BindReplayNetSendRecv_IndependentLocal<TIndependentTickReplicator_SP<ModelDef>>(ID, RepProxies.Replay, DataStore, Role);
				
				npCheckf(FNetworkPredictionDriver<ModelDef>::HasSimulation(), TEXT("AP must have Simulation."));
				npCheckf(FNetworkPredictionDriver<ModelDef>::HasInput(), TEXT("AP sim doesn't have Input?"));

				ServiceMask |= ENetworkPredictionService::IndependentLocalInput;
				ServiceMask |= ENetworkPredictionService::IndependentLocalTick;
				ServiceMask |= ENetworkPredictionService::IndependentLocalFinalize;

				ServiceMask |= ENetworkPredictionService::ServerRPC;
				ServiceMask |= ENetworkPredictionService::IndependentRollback;
				break;
			}
			case ENetRole::ROLE_SimulatedProxy:
			{
				BindClientNetRecv_Independent<TIndependentTickReplicator_AP<ModelDef>>(ID, RepProxies.AutonomousProxy, DataStore, Role);
				BindClientNetRecv_Independent<TIndependentTickReplicator_SP<ModelDef>>(ID, RepProxies.SimulatedProxy, DataStore, Role);
				BindReplayNetSendRecv_IndependentLocal<TIndependentTickReplicator_SP<ModelDef>>(ID, RepProxies.Replay, DataStore, Role);

				// Interpolation is the only supported mode for independently ticked SP simulations
				// (will add support for sim-extrapolate eventually)
				ServiceMask |= ENetworkPredictionService::IndependentInterpolate;
				break;
			}
		};
	}
	else if (Archetype.TickingMode == ENetworkPredictionTickingPolicy::Fixed)
	{
		// Point cached view to the FixedTickState's pending frame
		InstanceData.Info.View->UpdateView(FixedTickState.PendingFrame + FixedTickState.Offset,
			FixedTickState.GetTotalSimTimeMS(),
			&FrameData.Buffer[FixedTickState.PendingFrame].InputCmd, 
			&FrameData.Buffer[FixedTickState.PendingFrame].SyncState, 
			&FrameData.Buffer[FixedTickState.PendingFrame].AuxState);

		// Bind NetSend/Recv and role-dependent services
		switch (Role)
		{
			case ENetRole::ROLE_Authority:
			{
				if (FNetworkPredictionDriver<ModelDef>::HasSimulation())
				{
					BindServerNetRecv_Fixed<ModelDef>(ID, RepProxies.ServerRPC, DataStore);
					BindNetSend_Fixed<TFixedTickReplicator_AP<ModelDef>>(ID, RepProxies.AutonomousProxy, DataStore);

					if (FNetworkPredictionDriver<ModelDef>::HasInput())
					{
						ServiceMask |= bHasNetConnection ? ENetworkPredictionService::FixedInputRemote : ENetworkPredictionService::FixedInputLocal;
					}
				}
				
				BindNetSend_Fixed<TFixedTickReplicator_SP<ModelDef>>(ID, RepProxies.SimulatedProxy, DataStore);
				BindReplayNetSendRecv_Fixed<TFixedTickReplicator_SP<ModelDef>>(ID, RepProxies.Replay, DataStore, Role);
				break;
			}
			case ENetRole::ROLE_AutonomousProxy:
			{
				npCheckf(FNetworkPredictionDriver<ModelDef>::HasSimulation(), TEXT("AP must have Simulation."));
				npCheckf(FNetworkPredictionDriver<ModelDef>::HasInput(), TEXT("AP sim doesn't have Input?"));

				BindClientNetRecv_Fixed<TFixedTickReplicator_AP<ModelDef>>(ID, RepProxies.AutonomousProxy, DataStore, Role);
				BindClientNetRecv_Fixed<TFixedTickReplicator_SP<ModelDef>>(ID, RepProxies.SimulatedProxy, DataStore, Role);

				BindNetSend_Fixed<TFixedTickReplicator_Server<ModelDef>>(ID, RepProxies.ServerRPC, DataStore);
				BindReplayNetSendRecv_Fixed<TFixedTickReplicator_SP<ModelDef>>(ID, RepProxies.Replay, DataStore, Role);
				
				// Poll local input and send to server services
				ServiceMask |= ENetworkPredictionService::FixedInputLocal;
				ServiceMask |= ENetworkPredictionService::ServerRPC;
				break;
			}
			case ENetRole::ROLE_SimulatedProxy:
			{
				BindClientNetRecv_Fixed<TFixedTickReplicator_AP<ModelDef>>(ID, RepProxies.AutonomousProxy, DataStore, Role);
				BindClientNetRecv_Fixed<TFixedTickReplicator_SP<ModelDef>>(ID, RepProxies.SimulatedProxy, DataStore, Role);

				BindReplayNetSendRecv_Fixed<TFixedTickReplicator_SP<ModelDef>>(ID, RepProxies.Replay, DataStore, Role);
				break;
			}
		};

		// Authority vs Non-Authority services
		if (Role == ROLE_Authority)
		{
			if (FNetworkPredictionDriver<ModelDef>::HasSimulation())
			{
				ServiceMask |= ENetworkPredictionService::FixedTick;
				ServiceMask |= ENetworkPredictionService::FixedFinalize;

				if (FNetworkPredictionDriver<ModelDef>::HasFinalizeSmoothingFrame && Settings.bEnableFixedTickSmoothing)
				{
					ServiceMask |= ENetworkPredictionService::FixedSmoothing;
				}
				
			}
		}
		else
		{
			// These services depend on NetworkLOD for non authority cases
			switch(Config.NetworkLOD)
			{
			case ENetworkLOD::ForwardPredict:
				ServiceMask |= ENetworkPredictionService::FixedRollback;

				if (FNetworkPredictionDriver<ModelDef>::HasSimulation())
				{
					ServiceMask |= ENetworkPredictionService::FixedTick;
					ServiceMask |= ENetworkPredictionService::FixedFinalize;

					if (FNetworkPredictionDriver<ModelDef>::HasFinalizeSmoothingFrame && Settings.bEnableFixedTickSmoothing)
					{
						ServiceMask |= ENetworkPredictionService::FixedSmoothing;
					}
				}

				break;

			case ENetworkLOD::Interpolated:
				ServiceMask |= ENetworkPredictionService::FixedInterpolate;
				break;
			}
		}
	}

	// Net Cues: set which replicated cues we should accept based on if we are FP or interpolated
	if (Role != ROLE_Authority)
	{
		if (EnumHasAnyFlags(ServiceMask, ENetworkPredictionService::FixedInterpolate | ENetworkPredictionService::IndependentInterpolate))
		{
			InstanceData.CueDispatcher->SetReceiveReplicationTarget(ENetSimCueReplicationTarget::Interpolators);
		}
		else
		{
			InstanceData.CueDispatcher->SetReceiveReplicationTarget( Role == ROLE_AutonomousProxy ? ENetSimCueReplicationTarget::AutoProxy : ENetSimCueReplicationTarget::SimulatedProxy);
		}
	}

	// Register with selected services
	if (!bLockServices)
	{
		Services.RegisterInstance<ModelDef>(ID, InstanceData, ServiceMask);
	}
	else
	{
		DeferredServiceConfigDelegate.AddLambda([ID, ServiceMask](UNetworkPredictionWorldManager* Manager)
		{
			TModelDataStore<ModelDef>* DataStore = Manager->Services.GetDataStore<ModelDef>();
			TInstanceData<ModelDef>& InstanceData = *DataStore->Instances.Find(ID);
			Manager->Services.RegisterInstance<ModelDef>(ID, InstanceData, ServiceMask);
		});
	}
	
	// Call into driver to seed initial state if this is a new instance
	if (bNewInstance)
	{
		UE_NP_TRACE_SIM_CREATED(ID, InstanceData.Info.Driver, ModelDef);
		if (FNetworkPredictionDriver<ModelDef>::HasNpState())
		{
			FNetworkPredictionDriver<ModelDef>::InitializeSimulationState(InstanceData.Info.Driver, InstanceData.Info.View);
		}
	}
	else if (PrevPendingFrame != InstanceData.Info.View->PendingFrame)
	{
		// Not a new instance but PendingFrame changed, so copy contents from previous PendingFrame
		if (FNetworkPredictionDriver<ModelDef>::HasNpState())
		{
			FrameData.Buffer[InstanceData.Info.View->PendingFrame] = FrameData.Buffer[PrevPendingFrame];
		}
	}

	UE_NP_TRACE_SIM_CONFIG(ID.GetTraceID(), Role, bHasNetConnection, Archetype, Config, ServiceMask);
}

// ---------------------------------------------------------------------------------------
//	Server Receive Bindings
//		-Binds RepProxy serialize lambda to a Replicator function
//		-2 versions for Fixed/Independent
//		-Binds directly to TFixedTickReplicator_Server/TIndependentTickReplicator_Server
// ---------------------------------------------------------------------------------------
template<typename ModelDef>
void UNetworkPredictionWorldManager::BindServerNetRecv_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore)
{
	if (!RepProxy)
		return;

	const int32 ServerRecvIdx = DataStore->ServerRecv.GetIndex(ID);

	TFixedTickReplicator_Server<ModelDef>::SetNumInputsPerSend(Settings.FixedTickInputSendCount);

	TServerRecvData_Fixed<ModelDef>& ServerRecvData = DataStore->ServerRecv.GetByIndexChecked(ServerRecvIdx);
	ServerRecvData.TraceID = ID.GetTraceID();

	FFixedTickState* TickState = &this->FixedTickState;
	RepProxy->NetSerializeFunc = [DataStore, ServerRecvIdx, TickState](const FNetSerializeParams& P)
	{
		npEnsure(P.Ar.IsLoading());
		TServerRecvData_Fixed<ModelDef>& ServerRecvData = DataStore->ServerRecv.GetByIndexChecked(ServerRecvIdx);
		
		UE_NP_TRACE_SIM(ServerRecvData.TraceID);
		TFixedTickReplicator_Server<ModelDef>::NetRecv(P, ServerRecvData, DataStore, TickState);
	};
}

template<typename ModelDef>
void UNetworkPredictionWorldManager::BindServerNetRecv_Independent(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore)
{
	if (!RepProxy)
		return;

	const int32 ServerRecvIdx = DataStore->ServerRecv_IndependentTick.GetIndex(ID);

	TIndependentTickReplicator_Server<ModelDef>::SetNumInputsPerSend(Settings.IndependentTickInputSendCount);

	TServerRecvData_Independent<ModelDef>& ServerRecvData = DataStore->ServerRecv_IndependentTick.GetByIndexChecked(ServerRecvIdx);
	ServerRecvData.TraceID = ID.GetTraceID();
	ServerRecvData.InstanceIdx = DataStore->Instances.GetIndex(ID);
	ServerRecvData.FramesIdx = DataStore->Frames.GetIndex(ID);
	ServerRecvData.PendingFrame = 0;
	ServerRecvData.TotalSimTimeMS = 0;
	ServerRecvData.UnspentTimeMS = 0.f;
	ServerRecvData.LastConsumedFrame = INDEX_NONE;
	ServerRecvData.LastRecvFrame = INDEX_NONE;

	RepProxy->NetSerializeFunc = [DataStore, ServerRecvIdx](const FNetSerializeParams& P)
	{
		npEnsure(P.Ar.IsLoading());		
		TServerRecvData_Independent<ModelDef>& ServerRecvData = DataStore->ServerRecv_IndependentTick.GetByIndexChecked(ServerRecvIdx);

		UE_NP_TRACE_SIM(ServerRecvData.TraceID);
		TIndependentTickReplicator_Server<ModelDef>::NetRecv(P, ServerRecvData, DataStore);
	};
}

// ---------------------------------------------------------------------------------------
//	Client Receive Bindings
//		-Binds RepProxy serialize lambda to a Replicator function
//		-2 versions for Fixed/Independent
//		-Bind to either AP, or SP versions based on ReplicatorType
//			TFixedTickReplicator_AP
//			TFixedTickReplicator_SP
//			TIndependentTickReplicator_AP
//			TIndependentTickReplicator_SP
// ---------------------------------------------------------------------------------------

template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindClientNetRecv_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole)
{
	if (!RepProxy) 
		return;

	TFixedTickReplicator_Server<ModelDef>::SetNumInputsPerSend(Settings.FixedTickInputSendCount);

	const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndex(ID);
	NpResizeAndSetBit(DataStore->ClientRecvBitMask, ClientRecvIdx, false);

	TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
	InitClientRecvData<ModelDef>(ID, ClientRecvData, DataStore, NetRole);

	FFixedTickState* TickState = &this->FixedTickState;
	RepProxy->NetSerializeFunc = [DataStore, ClientRecvIdx, TickState](const FNetSerializeParams& P)
	{
		npEnsure(P.Ar.IsLoading());
		DataStore->ClientRecvBitMask[ClientRecvIdx] = true;
		auto& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);

		UE_NP_TRACE_SIM(ClientRecvData.TraceID);
		ReplicatorType::NetRecv(P, ClientRecvData, DataStore, TickState);
	};
}

template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindClientNetRecv_Independent(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole)
{
	if (!RepProxy) 
		return;

	TIndependentTickReplicator_Server<ModelDef>::SetNumInputsPerSend(Settings.IndependentTickInputSendCount);

	const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndex(ID);
	NpResizeAndSetBit(DataStore->ClientRecvBitMask, ClientRecvIdx, false);

	TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
	InitClientRecvData<ModelDef>(ID, ClientRecvData, DataStore, NetRole);

	FVariableTickState* TickState = &this->VariableTickState;
	RepProxy->NetSerializeFunc = [DataStore, ClientRecvIdx, TickState](const FNetSerializeParams& P)
	{
		npEnsure(P.Ar.IsLoading());
		DataStore->ClientRecvBitMask[ClientRecvIdx] = true;
		auto& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);

		UE_NP_TRACE_SIM(ClientRecvData.TraceID);
		ReplicatorType::NetRecv(P, ClientRecvData, DataStore, TickState);
	};
}

// ---------------------------------------------------------------------------------------
//	Send Bindings
//		-Binds RepProxy serialize lambda to a Replicator function
//		-3 versions: Fixed, Independent (Local Tick), Independent (Remote Ticked)
//		-Bind to either Server, AP, or SP versions based on ReplicatorType:
//			TFixedTickReplicator_Server
//			TIndependetTickReplicator_Server
//			TFixedTickReplicator_AP
//			TFixedTickReplicator_SP
//			TIndependentTickReplicator_AP
//			TIndependentTickReplicator_SP
// ---------------------------------------------------------------------------------------

template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindNetSend_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore)
{
	if (!RepProxy) 
		return;

	// Tick is based on local FixedTickState
	FFixedTickState* TickState = &this->FixedTickState;
	RepProxy->NetSerializeFunc = [ID, DataStore, TickState](const FNetSerializeParams& P)
	{
		npEnsure(P.Ar.IsSaving());
		UE_NP_TRACE_SIM(ID.GetTraceID());
		ReplicatorType::NetSend(P, ID, DataStore, TickState);
	};
}

template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindNetSend_IndependentLocal(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore)
{
	if (!RepProxy) 
		return;

	// Tick is based on local VariableTickState
	FVariableTickState* TickState = &this->VariableTickState;
	RepProxy->NetSerializeFunc = [ID, DataStore, TickState](const FNetSerializeParams& P)
	{
		npEnsure(P.Ar.IsSaving());
		UE_NP_TRACE_SIM(ID.GetTraceID());
		ReplicatorType::NetSend(P, ID, DataStore, TickState);
	};
}

template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindNetSend_IndependentRemote(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore)
{
	if (!RepProxy) 
		return;
	
	// Tick is based on ServerRecv_IndependentTick data
	const int32 ServerRecvIdx = DataStore->ServerRecv_IndependentTick.GetIndex(ID);
	RepProxy->NetSerializeFunc = [ID, this, DataStore, ServerRecvIdx](const FNetSerializeParams& P)
	{
		npEnsureSlow(P.Ar.IsSaving());
		TServerRecvData_Independent<ModelDef>& ServerRecv = DataStore->ServerRecv_IndependentTick.GetByIndexChecked(ServerRecvIdx);
		UE_NP_TRACE_SIM(ID.GetTraceID());
		ReplicatorType::NetSend(P, ID, DataStore, ServerRecv, &this->VariableTickState);
	};
}

// ---------------------------------------------------------------------------------------
//	Replay Bindings
//		-Binds RepProxy serialize lambda to a Replicator function
//		-3 versions: Fixed, Independent (Local Tick), Independent (Remote Ticked)
// ---------------------------------------------------------------------------------------

template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindReplayNetSendRecv_Fixed(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole)
{
	if (!RepProxy)
	{
		return;
	}

	const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndex(ID);
	NpResizeAndSetBit(DataStore->ClientRecvBitMask, ClientRecvIdx, false);

	TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
	InitClientRecvData<ModelDef>(ID, ClientRecvData, DataStore, NetRole);

	FFixedTickState* TickState = &this->FixedTickState;
	RepProxy->NetSerializeFunc = [ID, DataStore, ClientRecvIdx, TickState](const FNetSerializeParams& P)
	{
		if (P.Ar.IsLoading()) // Receiving replay data
		{
			DataStore->ClientRecvBitMask[ClientRecvIdx] = true;
			auto& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);

			UE_NP_TRACE_SIM(ClientRecvData.TraceID);
			ReplicatorType::NetRecv(P, ClientRecvData, DataStore, TickState);
		}
		else // Sending replay data
		{
			UE_NP_TRACE_SIM(ID.GetTraceID());
			ReplicatorType::NetSend(P, ID, DataStore, TickState);
		}
	};
}


template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindReplayNetSendRecv_IndependentLocal(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole)
{
	if (!RepProxy)
	{
		return;
	}

	const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndex(ID);
	NpResizeAndSetBit(DataStore->ClientRecvBitMask, ClientRecvIdx, false);

	TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
	InitClientRecvData<ModelDef>(ID, ClientRecvData, DataStore, NetRole);

	FVariableTickState* TickState = &this->VariableTickState;
	RepProxy->NetSerializeFunc = [ID, DataStore, ClientRecvIdx, TickState](const FNetSerializeParams& P)
	{
		if (P.Ar.IsLoading())	// Receiving replay data
		{
			DataStore->ClientRecvBitMask[ClientRecvIdx] = true;
			auto& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);

			UE_NP_TRACE_SIM(ClientRecvData.TraceID);
			ReplicatorType::NetRecv(P, ClientRecvData, DataStore, TickState);
		}
		else // Sending replay data
		{
			UE_NP_TRACE_SIM(ID.GetTraceID());
			ReplicatorType::NetSend(P, ID, DataStore, TickState);
		}
	};
}


template<typename ReplicatorType, typename ModelDef>
void UNetworkPredictionWorldManager::BindReplayNetSendRecv_IndependentRemote(FNetworkPredictionID ID, FReplicationProxy* RepProxy, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole)
{
	if (!RepProxy)
	{
		return;
	}

	const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndex(ID);
	NpResizeAndSetBit(DataStore->ClientRecvBitMask, ClientRecvIdx, false);

	TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
	InitClientRecvData<ModelDef>(ID, ClientRecvData, DataStore, NetRole);

	FVariableTickState* TickState = &this->VariableTickState;

	const int32 ServerRecvIdx = DataStore->ServerRecv_IndependentTick.GetIndex(ID);

	RepProxy->NetSerializeFunc = [ID, DataStore, ClientRecvIdx, TickState, ServerRecvIdx](const FNetSerializeParams& P)
	{
		if (P.Ar.IsLoading())	// Receiving replay data
		{
			DataStore->ClientRecvBitMask[ClientRecvIdx] = true;
			auto& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);

			UE_NP_TRACE_SIM(ClientRecvData.TraceID);
			ReplicatorType::NetRecv(P, ClientRecvData, DataStore, TickState);
		}
		else // Sending replay data
		{
			TServerRecvData_Independent<ModelDef>& ServerRecv = DataStore->ServerRecv_IndependentTick.GetByIndexChecked(ServerRecvIdx);
			UE_NP_TRACE_SIM(ID.GetTraceID());
			ReplicatorType::NetSend(P, ID, DataStore, ServerRecv, TickState);
		}
	};
}

// -----------------

template<typename ModelDef>
void UNetworkPredictionWorldManager::InitClientRecvData(FNetworkPredictionID ID, TClientRecvData<ModelDef>& ClientRecvData, TModelDataStore<ModelDef>* DataStore, ENetRole NetRole)
{
	ClientRecvData.TraceID = ID.GetTraceID();
	ClientRecvData.InstanceIdx = DataStore->Instances.GetIndexChecked(ID);
	ClientRecvData.FramesIdx = DataStore->Frames.GetIndexChecked(ID);
	ClientRecvData.NetRole = NetRole;
}

#undef UE_API
