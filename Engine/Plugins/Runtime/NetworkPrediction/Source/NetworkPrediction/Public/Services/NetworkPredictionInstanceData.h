// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Engine/EngineTypes.h"
#include "NetworkPredictionBuffer.h"
#include "NetworkPredictionConditionalState.h"
#include "NetworkPredictionInstanceMap.h"
#include "NetworkPredictionCues.h"
#include "NetworkPredictionModelDef.h"

// Enum that maps to internal NetworkPrediction services, see notes in NetworkPredictionServiceRegistry.h
enum class ENetworkPredictionService : uint32
{
	None = 0,

	// Common services that fix/independent can share
	ServerRPC				= 1 << 0,
	MAX_COMMON				= ServerRPC,

	// Services exclusive to fix tick mode
	FixedRollback				= 1 << 1,
	FixedExtrapolate			= 1 << 2,	// TODO
	FixedInterpolate			= 1 << 3,
	FixedInputLocal				= 1 << 4,
	FixedInputRemote			= 1 << 5,
	FixedTick					= 1 << 6,
	FixedSmoothing				= 1 << 7,
	FixedFinalize				= 1 << 8,
	MAX_FIXED					= FixedFinalize,

	// Services exclusive to independent tick mode
	IndependentRollback		= 1 << 10,
	IndependentExtrapolate	= 1 << 11,	// TODO
	IndependentInterpolate	= 1 << 12,

	IndependentLocalInput	= 1 << 13,
	IndependentLocalTick	= 1 << 14,
	IndependentRemoteTick	= 1 << 15,
	
	IndependentSmoothingFinalize	= 1 << 16,	// TODO
	IndependentLocalFinalize		= 1 << 17,
	IndependentRemoteFinalize		= 1 << 18,
	MAX_INDEPENDENT					= IndependentRemoteFinalize,

	// Helper masks
	ANY_COMMON = (MAX_COMMON<<1)-1,
	ANY_FIXED = ((MAX_FIXED<<1)-1) & ~ANY_COMMON,
	ANY_INDEPENDENT = (((MAX_INDEPENDENT<<1)-1) & ~ANY_FIXED) & ~ANY_COMMON,
};

ENUM_CLASS_FLAGS(ENetworkPredictionService);

// Basic data that all instances have
template<typename ModelDef=FNetworkPredictionModelDef>
struct TInstanceData
{
	TNetworkPredictionModelInfo<ModelDef>		Info;
	
	ENetRole NetRole = ROLE_None;
	TUniqueObj<TNetSimCueDispatcher<ModelDef>>	CueDispatcher;	// Should maybe be moved out?

	int32 TraceID;
	ENetworkPredictionService ServiceMask = ENetworkPredictionService::None;
};

// Frame data that instances with StateTypes will have.
template<typename ModelDef=FNetworkPredictionModelDef>
struct TInstanceFrameState
{
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	struct FFrame
	{
		TConditionalState<InputType>	InputCmd;
		TConditionalState<SyncType>	SyncState;
		TConditionalState<AuxType>	AuxState;
	};

	TNetworkPredictionBuffer<FFrame> Buffer;

	TInstanceFrameState()
		: Buffer(64) { } // fixme
};

// Data the client receives from the server
template<typename ModelDef=FNetworkPredictionModelDef>
struct TClientRecvData
{
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	int32 ServerFrame; // Fixed tick || Independent AP only
	int32 SimTimeMS; // Independent tick only

	TConditionalState<InputType> InputCmd; // SP Only
	TConditionalState<SyncType>	SyncState;
	TConditionalState<AuxType>	AuxState;

	// Acceleration data.
	int32 TraceID = INDEX_NONE;
	int32 InstanceIdx = INDEX_NONE;	// Index into TModelDataStore::Instances
	int32 FramesIdx = INDEX_NONE;	// Index into TModelDataStore::Frames
	ENetRole NetRole = ROLE_None;
};

// Data the server receives from a fixed ticking AP client
template<typename ModelDef=FNetworkPredictionModelDef>
struct TServerRecvData_Fixed
{
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;

	TNetworkPredictionBuffer<TConditionalState<InputType>> InputBuffer;

	// Note that these are client frame numbers, they do not match the servers local PendingFrame
	int32 LastConsumedFrame = INDEX_NONE;
	int32 LastRecvFrame = INDEX_NONE;
	
	int32 TraceID = INDEX_NONE;

	TServerRecvData_Fixed()
		: InputBuffer(32) {} // fixme
};

// Data the server receives from an independent ticking AP client
template<typename ModelDef=FNetworkPredictionModelDef>
struct TServerRecvData_Independent
{
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;

	struct FFrame
	{
		TConditionalState<InputType>	InputCmd;
		int32	DeltaTimeMS;
	};

	TServerRecvData_Independent()
		: InputBuffer(16) { }

	int32 PendingFrame = 0;
	int32 TotalSimTimeMS = 0;
	float UnspentTimeMS = 0.f;

	int32 LastConsumedFrame = INDEX_NONE;
	int32 LastRecvFrame = INDEX_NONE;

	TNetworkPredictionBuffer<FFrame> InputBuffer;

	// Acceleration data.
	int32 TraceID = INDEX_NONE;
	int32 InstanceIdx = INDEX_NONE;	// Index into TModelDataStore::Instances
	int32 FramesIdx = INDEX_NONE;	// Index into TModelDataStore::Frames
};

// Stores all public data for a given model def
template<typename ModelDef=FNetworkPredictionModelDef>
struct TModelDataStore
{
	TStableInstanceMap<TInstanceData<ModelDef>>	Instances;

	TInstanceMap<TInstanceFrameState<ModelDef>> Frames;
	
	TInstanceMap<TClientRecvData<ModelDef>> ClientRecv;
	TBitArray<> ClientRecvBitMask;

	TInstanceMap<TServerRecvData_Fixed<ModelDef>> ServerRecv;

	TInstanceMap<TServerRecvData_Independent<ModelDef>> ServerRecv_IndependentTick;
};
