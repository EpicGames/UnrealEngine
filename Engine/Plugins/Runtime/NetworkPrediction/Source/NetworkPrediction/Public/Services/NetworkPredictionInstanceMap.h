// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/SparseArray.h"
#include "NetworkPredictionID.h"

// Generic associative container for instance data keyed off of the FNetworkPredictionID
// SortedMap + SparseArray provides:
//	-O(log n) add/remove. 
//	-Stable index that can be cached for O(1) lookup in various service acceleration structures
//	-Data* are not stable out of this data structure
template<typename T>
struct TInstanceMap
{
	T& FindOrAdd(const FNetworkPredictionID& ID)
	{
		return Data[GetIndex(ID)];
	}

	T* Find(const FNetworkPredictionID& ID)
	{
		FIndex* MappedIdx = Lookup.Find((int32)ID);
		if (MappedIdx)
		{
			npEnsure(MappedIdx->idx != INDEX_NONE);
			return &Data[MappedIdx->idx];
		}
		return nullptr;
	}

	void Remove(const FNetworkPredictionID& ID)
	{
		FIndex RemovedIdx;
		if (Lookup.RemoveAndCopyValue((int32)ID, RemovedIdx))
		{
			Data.RemoveAt(RemovedIdx.idx);
			LastFreeIndex = FMath::Min(LastFreeIndex, RemovedIdx.idx);
		}
	}

	int32 GetIndexChecked(const FNetworkPredictionID& ID)
	{
		return Lookup.FindChecked((int32)ID).idx;
	}

	int32 GetIndex(const FNetworkPredictionID& ID)
	{
		FIndex& MappedIdx = Lookup.FindOrAdd((int32)ID);
		if (MappedIdx.idx == INDEX_NONE)
		{
			FSparseArrayAllocationInfo AllocationInfo = Data.AddUninitializedAtLowestFreeIndex(LastFreeIndex);
			MappedIdx.idx = AllocationInfo.Index;
			new (AllocationInfo.Pointer) T();
		}
		return MappedIdx.idx;
	}

	T& GetByIndexChecked(int32 idx)
	{
		npCheckSlow(Data.IsValidIndex(idx));
		return Data[idx];
	}

private:

	struct FIndex { int32 idx=INDEX_NONE; }; // just to default to INDEX_NONE for new entries
	TSortedMap<int32, FIndex> Lookup; // Sorted map of priority (spawnID -> stable index into SparseArray)
	int32 LastFreeIndex = 0;

	TSparseArray<T> Data;
};

template<typename T>
struct TStableInstanceMap
{
	T& FindOrAdd(const FNetworkPredictionID& ID)
	{
		return Data[GetIndex(ID)].Get();
	}

	T* Find(const FNetworkPredictionID& ID)
	{
		FIndex* MappedIdx = Lookup.Find((int32)ID);
		if (MappedIdx)
		{
			npEnsure(MappedIdx->idx != INDEX_NONE);
			return &Data[MappedIdx->idx].Get();
		}
		return nullptr;
	}

	void Remove(const FNetworkPredictionID& ID)
	{
		FIndex RemovedIdx;
		if (Lookup.RemoveAndCopyValue((int32)ID, RemovedIdx))
		{
			Data.RemoveAt(RemovedIdx.idx);
			LastFreeIndex = FMath::Min(LastFreeIndex, RemovedIdx.idx);
		}
	}

	int32 GetIndexChecked(const FNetworkPredictionID& ID)
	{
		return Lookup.FindChecked((int32)ID).idx;
	}

	int32 GetIndex(const FNetworkPredictionID& ID)
	{
		FIndex& MappedIdx = Lookup.FindOrAdd((int32)ID);
		if (MappedIdx.idx == INDEX_NONE)
		{
			FSparseArrayAllocationInfo AllocationInfo = Data.AddUninitializedAtLowestFreeIndex(LastFreeIndex);
			MappedIdx.idx = AllocationInfo.Index;
			new (AllocationInfo.Pointer) TUniqueObj<T>();
		}
		return MappedIdx.idx;
	}

	T& GetByIndexChecked(int32 idx)
	{
		npCheckSlow(Data.IsValidIndex(idx));
		return Data[idx].Get();
	}

private:

	struct FIndex { int32 idx=INDEX_NONE; }; // just to default to INDEX_NONE for new entries
	TSortedMap<int32, FIndex> Lookup; // Sorted map of priority (spawnID -> stable index into SparseArray)
	int32 LastFreeIndex = 0;

	TSparseArray<TUniqueObj<T>> Data;
};
