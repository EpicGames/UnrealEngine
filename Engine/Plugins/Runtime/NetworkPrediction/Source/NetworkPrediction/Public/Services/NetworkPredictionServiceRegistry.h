// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "NetworkPredictionService_Ticking.inl"
#include "NetworkPredictionService_Rollback.inl"
#include "NetworkPredictionService_Interpolate.inl"
#include "NetworkPredictionService_Input.inl"
#include "NetworkPredictionService_Finalize.inl"
#include "NetworkPredictionService_ServerRPC.inl"
#include "NetworkPredictionService_Smooth.inl"

// Services do the actual system work on batches of registered instances. UNetworkPredictionWorldManager orchestrates them.
//
// Services should treat all registered instances the same in most cases. Instead of 1 services that has different behavior based 
// on Role/NetConnection/etc, make N services to cover each unique case. There will be exceptions to this where a role branch on a minor 
// aspect of the service is better than creating a brand new service.
//
// Services are defined by an interface (E.g, IFixedRollbackService) and a ModelDef based template implementation (E.g, TFixedRollbackService).
//
// Services operate on the data stored in TModelDataStore, which is owned by the FNetworkPredictionServiceRegistry. 
// All services get a pointer to the data store when created.
// Services are free to keep internal acceleration structures but should mainly operate on the per-instance data in the data store.
// 
// FNetworkPredictionServiceRegistry maintains lists of all active services. Services are bound to the ENetworkPredictionService enum for identification.
// The template implementations are instantiated on demand during registration where ModelDef::ID is the index into the TServiceStorage array.
//
// The big picture looks like this, where 1=templated class has been instantiated and is managing X registered instances:
//
//                              [ModelDef::ID]
//	  [ServiceType]			0  1  2  3  4  5  ... N
//	FixedRollback           1  0  1  1  1  0
//	FixedExtrapolated       0  1  0  0  1  0
//	...
//	IndependentFinalize     0  0  0  0  0  1
//
//
// NPs instance are registered to X services based on their config and network role/connection. Subscribed services are stored in TInstanceData<ModelDef>::ServiceMask.
// As runtime settings change, E.g, NetworkLOD, the set of subscribed services will change. This is done efficiently with the ServiceMask.
//
// Services are an implementation detail of the UNetworkPredictionWorldManager and should not be exposed directly to outside runtime code. 
// E.g, don't pass pointers to services back to user code.
//
// Services can be specialized on ModelDef types. This could facilitate more efficient batch processing or further per-simulation/modeldef customization.
// Services should have RegisterInstance/UnregisterInstance functions that take only the FNetworkPredictionID as parameter.
//
// No direct cross-service communication. Services can have their own internal services (E.g, rollback has an internal tick service) but the "top level" services do not communicate. 
// UNetworkPredictionWorldManager should coordinate things.
//
//	
// Adding new services:
//	1. Add entry to ENetworkPredictionService
//	2. Add NP_DECLARE_SERVICE 
//	3. Add NP_DEFINE_SERVICE_CALL
//	4. Add logic to UNetworkPredictionWorldManager::ConfigureInstance to determine the conditions for subscribing to the service
//	5. Add logic in UNetworkPredictionWorldManager to actually invoke the service. This will obviously be service dependent.
//
// New services types are not meant to be arbitrarily added by projects. Adding a brand new service requires modification of UNetworkPredictionWorldManager.
// If you want to tack on 'something special', look at specializing an existing service (E.g the tick service could do 'extra stuff' per tick for example).
//
// Future Improvements:
//	-All services template classes are currently instantiated for all ModelDefs (the templated code is generated for each ModelDef)
//	-Even if ModelDefs are incompatible for a given service.
//	-To be clear: it does not instantiate an instance of TLocalInputService at runtime, but the code is generated and almost certainly cannot be culled by the compiler.
//	-Concepts could be used to determine if ModelDefs are compatible with Services and we could avoid the template code instantiation.
//	-This would add more template machinery, and basically requires all TService<ModelDefs> be hidden behind SFINAE walls. E.g, Factories for instantiating and CallOrNot for Register/Unregister.
//	-Its not clear if its worth the effort at this point. In practice there should be relatively few ModelDefs that would benefit from this. 
//	-But it could make a difference in some cases for build time and exe size

class FNetworkPredictionServiceRegistry
{
public:

	// -----------------------------------------------------------------------------------------
	//	Register / Unregister
	// -----------------------------------------------------------------------------------------

	// Registers instance with given services. Will unregister instance with any previously-subscribed services.
	template<typename ModelDef>
	void RegisterInstance(FNetworkPredictionID ID, TInstanceData<ModelDef>& InstanceData, ENetworkPredictionService ServiceMask)
	{
		// Expected to register for fixed XOR independent services
		npEnsureSlow(EnumHasAnyFlags(ServiceMask, ENetworkPredictionService::ANY_FIXED) ^ EnumHasAnyFlags(ServiceMask, ENetworkPredictionService::ANY_INDEPENDENT));
		if (InstanceData.ServiceMask != ENetworkPredictionService::None)
		{
			// Only unregister/register what is changing
			const ENetworkPredictionService UnregisterMask = InstanceData.ServiceMask & ~(ServiceMask);
			const ENetworkPredictionService RegisterMask = ServiceMask & ~(InstanceData.ServiceMask);

			UnregisterInstance_Internal<ModelDef>(ID, UnregisterMask);

			ForEachService<ModelDef>(RegisterMask, [ID](auto Ptr)
			{
				Ptr->RegisterInstance(ID);
			});
		}
		else
		{
			// Register with everything
			ForEachService<ModelDef>(ServiceMask, [ID](auto Ptr)
			{
				Ptr->RegisterInstance(ID);
			});
		}

		InstanceData.ServiceMask = ServiceMask;
	}

	template<typename ModelDef>
	void UnregisterInstance(FNetworkPredictionID ID)
	{
		TModelDataStore<ModelDef>* DataStore = GetDataStore<ModelDef>();
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.FindOrAdd(ID);
		const ENetworkPredictionService Mask = ENetworkPredictionService::None;

		UnregisterInstance_Internal<ModelDef>(ID, InstanceData.ServiceMask);
		InstanceData.ServiceMask = ENetworkPredictionService::None;
	}

	// -----------------------------------------------------------------------------------------
	//	DataStore
	// -----------------------------------------------------------------------------------------

	template<typename ModelDef=FNetworkPredictionModelDef>
	TModelDataStore<ModelDef>* GetDataStore()
	{
		npEnsureMsgf(ModelDef::ID > 0, TEXT("ModelDef %s has invalid ID assigned. Could be missing NP_MODEL_REGISTER."), ModelDef::GetName());

		struct FThisDataStore : IDataStore
		{
			TModelDataStore<ModelDef> Self;
		};

		if (DataStoreArray.IsValidIndex(ModelDef::ID) == false)
		{
			DataStoreArray.SetNum(ModelDef::ID+1);
		}

		TUniquePtr<IDataStore>& Item = DataStoreArray[ModelDef::ID];
		if (Item.IsValid() == false)
		{
			Item = MakeUnique<FThisDataStore>();
		}

		return &((FThisDataStore*)Item.Get())->Self;
	};

	// -----------------------------------------------------------------------------------------
	//	Services
	// -----------------------------------------------------------------------------------------

	template<typename InServiceInterface, int32 NumInlineServices=5>
	struct TServiceStorage
	{
		using ServiceInterface = InServiceInterface;
		TSparseArray<TUniquePtr<ServiceInterface>, TInlineSparseArrayAllocator<NumInlineServices>> Array;
	};

	// Macros are mainly to enforce consistent naming and cohesion with ENetworkPredictionService
#define NP_DECLARE_SERVICE(EnumName, ServiceInterface) TServiceStorage<ServiceInterface> EnumName
#define NP_DEFINE_SERVICE_CALL(EnumName, ServiceType) ConditionalCallFuncOnService<ENetworkPredictionService::EnumName, ServiceType<ModelDef>>(EnumName, Func, Mask)
	
	// Declares generic storage for the service type: TServiceStorage<InterfaceType>
	NP_DECLARE_SERVICE(ServerRPC,			IServerRPCService);

	NP_DECLARE_SERVICE(FixedRollback,				IFixedRollbackService);
	NP_DECLARE_SERVICE(FixedInterpolate,			IFixedInterpolateService);
	NP_DECLARE_SERVICE(FixedInputLocal,				IInputService);
	NP_DECLARE_SERVICE(FixedInputRemote,			IInputService);
	NP_DECLARE_SERVICE(FixedTick,					ILocalTickService);
	NP_DECLARE_SERVICE(FixedFinalize,				IFinalizeService);
	NP_DECLARE_SERVICE(FixedSmoothing,				IFixedSmoothingService);

	NP_DECLARE_SERVICE(IndependentRollback,			IIndependentRollbackService);
	NP_DECLARE_SERVICE(IndependentInterpolate,		IIndependentInterpolateService);
	NP_DECLARE_SERVICE(IndependentLocalInput,		IInputService);
	NP_DECLARE_SERVICE(IndependentLocalTick,		ILocalTickService);
	NP_DECLARE_SERVICE(IndependentRemoteTick,		IRemoteIndependentTickService);
	NP_DECLARE_SERVICE(IndependentLocalFinalize,	IFinalizeService);
	NP_DECLARE_SERVICE(IndependentRemoteFinalize,	IRemoteFinalizeService);
	
private:

	template<typename ModelDef=FNetworkPredictionModelDef, typename FuncRefType>
	void ForEachService(ENetworkPredictionService Mask, const FuncRefType& Func)
	{
		// Call to ConditionalCallFuncOnService
		NP_DEFINE_SERVICE_CALL(ServerRPC,			TServerRPCService);

		if (EnumHasAnyFlags(Mask, ENetworkPredictionService::ANY_FIXED))
		{
			NP_DEFINE_SERVICE_CALL(FixedRollback,				TFixedRollbackService);
			NP_DEFINE_SERVICE_CALL(FixedInterpolate,			TFixedInterpolateService);
			NP_DEFINE_SERVICE_CALL(FixedInputLocal,				TLocalInputService);
			NP_DEFINE_SERVICE_CALL(FixedInputRemote,			TRemoteInputService);
			NP_DEFINE_SERVICE_CALL(FixedTick,					TLocalTickService);
			NP_DEFINE_SERVICE_CALL(FixedFinalize,				TFinalizeService);
			NP_DEFINE_SERVICE_CALL(FixedSmoothing,				TFixedSmoothingService);
		}
		else if (EnumHasAnyFlags(Mask, ENetworkPredictionService::ANY_INDEPENDENT))
		{
			NP_DEFINE_SERVICE_CALL(IndependentRollback,			TIndependentRollbackService);
			NP_DEFINE_SERVICE_CALL(IndependentInterpolate,		TIndependentInterpolateService);
			NP_DEFINE_SERVICE_CALL(IndependentLocalInput,		TLocalInputService);
			NP_DEFINE_SERVICE_CALL(IndependentLocalTick,		TLocalTickService);
			NP_DEFINE_SERVICE_CALL(IndependentRemoteTick,		TRemoteIndependentTickService);
			NP_DEFINE_SERVICE_CALL(IndependentLocalFinalize,	TFinalizeService);
			NP_DEFINE_SERVICE_CALL(IndependentRemoteFinalize,	TRemoteFinalizeService);
		}
	}

	template<ENetworkPredictionService ServiceMask, typename ServiceType, typename FuncRefType, typename ServiceStorageType>
	void ConditionalCallFuncOnService(ServiceStorageType& ServiceStorage, const FuncRefType& Func, ENetworkPredictionService Mask)
	{
		using ModelDef = typename ServiceType::ModelDef;
		if (EnumHasAllFlags(Mask, ServiceMask))
		{
			// Resize array for this ModelDef if necessary
			if (ServiceStorage.Array.IsValidIndex(ModelDef::ID) == false)
			{
				FSparseArrayAllocationInfo AllocationInfo = ServiceStorage.Array.InsertUninitialized(ModelDef::ID);
				new (AllocationInfo.Pointer) TUniquePtr<typename ServiceStorageType::ServiceInterface>();
			}

			// Allocate instance on the UniquePtr if necessary
			auto& Ptr = ServiceStorage.Array[ModelDef::ID];
			if (Ptr.IsValid() == false)
			{
				Ptr = MakeUnique<ServiceType>(GetDataStore<ModelDef>());
			}

			Func((ServiceType*)Ptr.Get());
		}
	}

	template<typename ServiceType, typename ArrayType>
	ServiceType* GetService_Internal(ArrayType& Array)
	{
		using ModelDef = typename ServiceType::ModelDef;
		if (Array.IsValidIndex(ModelDef::ID) == false)
		{
			FSparseArrayAllocationInfo AllocationInfo = Array.InsertUninitialized(ModelDef::ID);
			new (AllocationInfo.Pointer) TUniquePtr<ServiceType>(new ServiceType(GetDataStore<ModelDef>()));
		}

		auto& Item = Array[ModelDef::ID];
		npCheckf(Item.IsValid(), TEXT("Service not initialized"));
		return (ServiceType*)Item.Get();
	}

	template<typename ModelDef>
	void UnregisterInstance_Internal(FNetworkPredictionID ID, ENetworkPredictionService ServiceMask)
	{
		ForEachService<ModelDef>(ServiceMask, [ID](auto Ptr)
		{
			Ptr->UnregisterInstance(ID);
		});
	}

	struct IDataStore
	{
		virtual ~IDataStore() = default;
	};

	TArray<TUniquePtr<IDataStore>>	DataStoreArray;
};
