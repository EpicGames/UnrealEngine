// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionDriver.h"
#include "NetworkPredictionUtil.h"
#include "Services/NetworkPredictionInstanceData.h"

class IFinalizeService
{
public:

	virtual ~IFinalizeService() = default;
	virtual void FinalizeFrame(float DeltaTimeSeconds, const int32 SimFrame, const int32 SimTimeMS, const int32 FixedStepMS) = 0;
};

template<typename InModelDef>
class TFinalizeService : public IFinalizeService
{
public:

	using ModelDef = InModelDef;
	using DriverType = typename ModelDef::Driver;
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	TFinalizeService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 InstanceIdx = DataStore->Instances.GetIndex(ID);
		NpResizeAndSetBit(FinalizeBitArray, InstanceIdx);
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 InstanceIdx = DataStore->Instances.GetIndex(ID);
		FinalizeBitArray[InstanceIdx] = false;
	}

	void FinalizeFrame(float DeltaTimeSeconds, const int32 ServerSimFrame, const int32 SimTimeMS, const int32 FixedStepMS) final override
	{
		for (TConstSetBitIterator<> BitIt(FinalizeBitArray); BitIt; ++BitIt)
		{
			TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(BitIt.GetIndex());

			npCheckSlow(InstanceData.Info.View);
			npCheckSlow(InstanceData.Info.View->PendingSyncState && InstanceData.Info.View->PendingAuxState);

			SyncType* SyncState = (SyncType*)InstanceData.Info.View->PendingSyncState;
			AuxType* AuxState = (AuxType*)InstanceData.Info.View->PendingAuxState;

			FNetworkPredictionDriver<ModelDef>::FinalizeFrame(InstanceData.Info.Driver, SyncState, AuxState);

			// Dispatch Cues: it may be better to take two passes here. Do all FinalizeFrame calls then all Dispatch Cues
			// (Dispatch Cues can go deep into user code, may be more cache efficient to take two passes).
			FNetworkPredictionDriver<ModelDef>::DispatchCues(&InstanceData.CueDispatcher.Get(), InstanceData.Info.Driver, ServerSimFrame, SimTimeMS, FixedStepMS);
		}
	}

private:

	TBitArray<> FinalizeBitArray; // index into DataStore->Instances
	TModelDataStore<ModelDef>* DataStore;
};

// ---------------------------------------------------------------------------------------

class IRemoteFinalizeService
{
public:

	virtual ~IRemoteFinalizeService() = default;
	virtual void FinalizeFrame(float DeltaTimeSeconds) = 0;
};

template<typename InModelDef>
class TRemoteFinalizeService : public IRemoteFinalizeService
{
public:

	using ModelDef = InModelDef;
	using DriverType = typename ModelDef::Driver;
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	TRemoteFinalizeService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 InstanceIdx = DataStore->ServerRecv_IndependentTick.GetIndex(ID);
		NpResizeAndSetBit(FinalizeBitArray, InstanceIdx);
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 InstanceIdx = DataStore->ServerRecv_IndependentTick.GetIndex(ID);
		FinalizeBitArray[InstanceIdx] = false;
	}

	void FinalizeFrame(float DeltaTimeSeconds) final override
	{
		for (TConstSetBitIterator<> BitIt(FinalizeBitArray); BitIt; ++BitIt)
		{
			TServerRecvData_Independent<ModelDef>& ServerRecvData = DataStore->ServerRecv_IndependentTick.GetByIndexChecked(BitIt.GetIndex());
			TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ServerRecvData.InstanceIdx);

			npCheckSlow(InstanceData.Info.View);
			npCheckSlow(InstanceData.Info.View->PendingSyncState && InstanceData.Info.View->PendingAuxState);

			SyncType* SyncState = (SyncType*)InstanceData.Info.View->PendingSyncState;
			AuxType* AuxState = (AuxType*)InstanceData.Info.View->PendingAuxState;

			FNetworkPredictionDriver<ModelDef>::FinalizeFrame(InstanceData.Info.Driver, SyncState, AuxState);

			// Dispatch Cues: it may be better to take two passes here. Do all FinalizeFrame calls then all Dispatch Cues
			// (Dispatch Cues can go deep into user code, may be more cache efficient to take two passes).
			FNetworkPredictionDriver<ModelDef>::DispatchCues(&InstanceData.CueDispatcher.Get(), InstanceData.Info.Driver, ServerRecvData.PendingFrame, ServerRecvData.TotalSimTimeMS, 0);
		}
	}

private:

	TBitArray<> FinalizeBitArray; // index into DataStore->ServerRecv_IndependentTick
	TModelDataStore<ModelDef>* DataStore;
};
