// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionTrace.h"
#include "Services/NetworkPredictionInstanceData.h"

// InputService's job is to write InputCmds to a subscribed instance's FrameBuffer[PendingFrame].InputCmd.

class IInputService
{
public:

	virtual ~IInputService() = default;
	virtual void ProduceInput(int32 DeltaTimeMS) = 0;
};

// Calls ProduceInput on driver to get local input
//	Requires valid FNetworkPredictionDriver::ProduceInput function
template<typename InModelDef>
class TLocalInputService : public IInputService
{
public:

	using ModelDef = InModelDef;
	using DriverType = typename ModelDef::Driver;
	using StateTypes = typename ModelDef::StateTypes;
	using InputType= typename StateTypes::InputType;


	TLocalInputService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		npCheckfSlow(!std::is_void_v<InputType>, TEXT("ModelDef %s with null InputCmd type was registered for local input service."), ModelDef::GetName());

		TInstanceData<ModelDef>* InstanceData = DataStore->Instances.Find(ID);
		npCheckSlow(InstanceData);

		InstanceMap.Add((int32)ID, FInstance{ID.GetTraceID(), InstanceData->Info.View, InstanceData->Info.Driver});
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		InstanceMap.Remove((int32)ID);
	}

	void ProduceInput(int32 DeltaTimeMS) final override
	{
		for (auto& MapIt : InstanceMap)
		{
			FInstance& Instance = MapIt.Value;

			npCheckSlow(Instance.Driver);
			npCheckSlow(Instance.View);
			npCheckSlow(Instance.View->PendingInputCmd);

			FNetworkPredictionDriver<ModelDef>::ProduceInput(Instance.Driver, DeltaTimeMS, (InputType*)Instance.View->PendingInputCmd);

			UE_NP_TRACE_PRODUCE_INPUT(Instance.TraceID);
			UE_NP_TRACE_USER_STATE_INPUT(ModelDef, (InputType*)Instance.View->PendingInputCmd);
		}
	}

private:
	
	struct FInstance
	{
		int32 TraceID;
		FNetworkPredictionStateView* View;
		DriverType* Driver;
	};

	TSortedMap<int32, FInstance, TInlineAllocator<1>> InstanceMap;
	TModelDataStore<ModelDef>* DataStore;
};

// Pulls input from ServerRecvData. E.g: server side input buffering
template<typename InModelDef>
class TRemoteInputService : public IInputService
{
public:

	using ModelDef = InModelDef;
	using DriverType = typename ModelDef::Driver;
	using StateTypes = typename ModelDef::StateTypes;
	using InputType= typename StateTypes::InputType;

	TRemoteInputService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	static void SetMaxFaultLimit(int32 InMaxFaultLimit) 
	{ 
		MaxFaultLimit = InMaxFaultLimit; 
	}

	void RegisterInstance(FNetworkPredictionID ID)
	{
		TInstanceData<ModelDef>* InstanceData = DataStore->Instances.Find(ID);
		npCheckSlow(InstanceData);
		
		const int32 ServerRecvIdx = DataStore->ServerRecv.GetIndex(ID);
		InstanceMap.Add((int32)ID, FInstance{ID.GetTraceID(), InstanceData->Info.View, ServerRecvIdx});
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		InstanceMap.Remove(ID);
	}

	void ProduceInput(int32 DeltaTimeMS) final override
	{
		for (auto& MapIt : InstanceMap)
		{
			FInstance& Remote = MapIt.Value;

			npCheckSlow(Remote.View);
			npCheckSlow(Remote.View->PendingInputCmd);
			npCheckSlow(Remote.View->PendingFrame >= 0);
			
			UE_NP_TRACE_PRODUCE_INPUT(Remote.TraceID);
			UE_NP_TRACE_PUSH_INPUT_FRAME(Remote.View->PendingFrame);

			TServerRecvData_Fixed<ModelDef>& ServerRecvData = DataStore->ServerRecv.GetByIndexChecked(Remote.ServerRecvIdx);
			if (ServerRecvData.LastRecvFrame == INDEX_NONE)
			{
				//UE_LOGF(LogNetworkPrediction, Warning, "[Remote.Input] None received");
				continue;
			}

			const int32 NumBufferedInputCmds = ServerRecvData.LastRecvFrame - ServerRecvData.LastConsumedFrame;

			// Check overflow
			if (NumBufferedInputCmds >= ServerRecvData.InputBuffer.Capacity())
			{
				UE_NP_TRACE_SYSTEM_FAULT("[Remote.Input] overflow %d %d -> %d", ServerRecvData.LastRecvFrame, ServerRecvData.LastConsumedFrame, ServerRecvData.LastRecvFrame - ServerRecvData.LastConsumedFrame);
				ServerRecvData.LastConsumedFrame = ServerRecvData.LastRecvFrame - ServerRecvData.InputBuffer.Capacity() + 1;
			}

			// Check fault - we are waiting for Cmds to reach FaultLimit before continuing
			if (Remote.bFault)
			{
				if (NumBufferedInputCmds < Remote.FaultLimit)
				{
					// Skip this instance because it is in fault. We will use the prev input for this frame.
					//UE_NP_TRACE_SYSTEM_FAULT("[Remote.Input]   Input in Fault. Total buffered cmds: %d/%d", NumBufferedInputCmds, Remote.FaultLimit);
					UE_NP_TRACE_BUFFERED_INPUT(NumBufferedInputCmds, true);
					continue;
				}

				UE_NP_TRACE_SYSTEM_FAULT("[Remote.Input] Recovered from Fault. FaultLimit now: %d", Remote.FaultLimit);
				Remote.bFault = false;
			}
			else if (NumBufferedInputCmds == 0)
			{
				// No Cmds to process, enter fault state. Increment FaultLimit each time this happens.
				// Note: time dilation will gradually bring any excessive input buffering back down as net latency improves. When dilation is enabled, the FaultLimit project setting can be kept to 1 safely.
				Remote.bFault = true;
				Remote.FaultLimit = FMath::Min(Remote.FaultLimit + 1, FMath::Min(MaxFaultLimit, ServerRecvData.InputBuffer.Capacity()));
				
				UE_NP_TRACE_SYSTEM_FAULT("[Remote.Input] New Fault. No cmds to process. New FaultLimit=%d", Remote.FaultLimit);
				UE_NP_TRACE_BUFFERED_INPUT(0, true);
				continue;
			}

			// Consume next InputCmd
			ServerRecvData.InputBuffer[++ServerRecvData.LastConsumedFrame].CopyTo((InputType*)Remote.View->PendingInputCmd);

			UE_NP_TRACE_BUFFERED_INPUT(NumBufferedInputCmds, false);
			UE_NP_TRACE_USER_STATE_INPUT(ModelDef, (InputType*)Remote.View->PendingInputCmd);
		}
	}

private:
	
	struct FInstance
	{
		int32 TraceID;
		FNetworkPredictionStateView* View;

		int32 ServerRecvIdx;  // idx into DataStore->ServerRecv::GetByIndexChecked
		int32 FaultLimit = 2; // InputBuffer must have >= this number of unprocessed commands before resuming consumption
		bool bFault=true;	  // Recently starved on input buffer, wait until we reach FaultLimit before consuming input again. (Note you start out in fault to let the buffer fill up)
	};

	TSortedMap<int32, FInstance> InstanceMap;
	TModelDataStore<ModelDef>* DataStore;
	
	inline static int32 MaxFaultLimit = 6;
};

