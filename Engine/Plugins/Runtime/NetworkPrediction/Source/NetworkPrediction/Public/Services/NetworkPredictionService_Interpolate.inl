// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionCVars.h"
#include "NetworkPredictionLog.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionTrace.h"
#include "NetworkPredictionUtil.h"
#include "Services/NetworkPredictionInstanceData.h"

// Interpolation does generic linear interpolation on received replicated data.
// Calls FinalizeFrame and DispatchCues every render frame for smooth motion and event dispatching.
// This is a client side service: the server does not interpolate anything.
// Advantage: client is never wrong. You will never see corrections on interpolated objects.
// Disadvantage: built in latency due to buffering replicated frame. Cannot predict interpolated simulations. 
// Requires consistent replication rate. Interpolated simulations are the farthest behind in time.
//
// Separate implementations for Fixed and Independent ticking modes.
//
// Interpolation Frame/Time is managed by the NetworkPredictionWorldManager.
// An important thing here is that all interpolated simulations are on the same timeline (either Fixed or Independent, we never correlate the two!)
// In other words, all interpolated simulations are "in sync": they have the same amount of buffered frames/time. 
//
//
//	Notes:
//		-We never allow extrapolation. If we under run an interpolation buffer we just use latest state ("cap at 100%")
//		-This could be allowed/opt in option on the ModelDef.


namespace NetworkPredictionCVars
{
	// These are pretty bare bones. Will move to Insights tracing in the future for interpolation debugging
	NETSIM_DEVCVAR_SHIPCONST_INT(PrintSyncInterpolation, 0, "np.Interpolation.PrintSync", "Print Sync State buffer during interpolation");
	NETSIM_DEVCVAR_SHIPCONST_INT(DrawInterpolation, 0, "np.Interpolation.Draw", "Draw interpolation debug state in world");

	NETSIM_DEVCVAR_SHIPCONST_INT(DisableInterpolation, 0, "np.Interpolation.Disable", "Disables smooth interpolation and just Finalizes the last received frame");
}


// ------------------------------------------------------------------------------
//	FixedTick Interpolation
//
//	Since all fix tick sims agree on frame number, most work can be shared.
//	The main concerns here are keeping valid data in the Frame buffer.
//	As we NetRecv, make sure we have a continuous line of valid frames between to ToFrame and received frame.
//	When interpolating, make sure we haven't been starved on replication.
//
//	Note: Interpolation ignores the TickState->Offset that forward predicted services use. Instead, we copy
//	into the local frame buffers @ the server frame numbers. 
// ------------------------------------------------------------------------------
class IFixedInterpolateService
{
public:

	virtual ~IFixedInterpolateService() = default;
	virtual void Reconcile(const FFixedTickState* TickState) = 0;
	virtual void FinalizeFrame(float DeltaTimeSeconds, const FFixedTickState* TickState) = 0;
};

template<typename InModelDef>
class TFixedInterpolateService : public IFixedInterpolateService
{
public:

	using ModelDef = InModelDef;
	using StateTypes = typename ModelDef::StateTypes;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;
	using SyncAuxType = TSyncAuxPair<StateTypes>;

	TFixedInterpolateService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);
		NpResizeAndSetBit(ClientRecvBitMask, ClientRecvIdx);

		const int32 InstanceDataIdx = DataStore->Instances.GetIndexChecked(ID);
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(InstanceDataIdx);
		
		
		FSparseArrayAllocationInfo AllocInfo = Instances.InsertUninitialized(ClientRecvIdx);
		new (AllocInfo.Pointer) FInstance(ID.GetTraceID(), InstanceDataIdx, DataStore->Frames.GetIndexChecked(ID));

		// Point the PresentationView to our managed state. Note this only has to be done once
		FInstance* InternalInstance = (FInstance*)AllocInfo.Pointer;
		InstanceData.Info.View->UpdatePresentationView(InternalInstance->SyncState, InternalInstance->AuxState);
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);

		const TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvData.InstanceIdx);
		InstanceData.Info.View->ClearPresentationView();

		FInstance& InterpolationData = Instances[ClientRecvIdx];
		if (InterpolationData.bTwoValidFrames == false)
		{
			// We hid this but never got a chance to unhide it (can happen in possession cases)
			FNetworkPredictionDriver<ModelDef>::SetHiddenForInterpolation(InstanceData.Info.Driver, false);
		}
		
		ClientRecvBitMask[ClientRecvIdx] = false;
		Instances.RemoveAt(ClientRecvIdx);
	}

	void Reconcile(const FFixedTickState* TickState) final override
	{
		const int32 FixedStepMS = TickState->FixedStepMS;
		const int32 FromFrame = TickState->Interpolation.ToFrame-1;
		const int32 ToFrame = TickState->Interpolation.ToFrame;

		npCheckSlow(ToFrame != INDEX_NONE); // Reconcile calls should be suppressed until interpolation begins

		// DataStore->ClientRecvBitMask size can change without us knowing so make sure out InstanceBitArray size stays in sync
		NpResizeBitArray(ClientRecvBitMask, DataStore->ClientRecvBitMask.Num());

		for (TConstDualSetBitIterator<FDefaultBitArrayAllocator,FDefaultBitArrayAllocator> BitIt(ClientRecvBitMask, DataStore->ClientRecvBitMask); BitIt; ++BitIt)
		{
			const int32 ClientRecvIdx = BitIt.GetIndex();
			TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ClientRecvData.FramesIdx);

			const int32 LocalFrame = ClientRecvData.ServerFrame;
			typename TInstanceFrameState<ModelDef>::FFrame& LocalFrameData = Frames.Buffer[LocalFrame];
			
			npEnsure(ToFrame == INDEX_NONE || LocalFrame - ToFrame < Frames.Buffer.Capacity());

			UE_NP_TRACE_SIM(ClientRecvData.TraceID);

			// Copy latest received state into local frame buffer
			LocalFrameData.SyncState = ClientRecvData.SyncState;
			LocalFrameData.AuxState = ClientRecvData.AuxState;

			FInstance& InterpolationData = Instances[ClientRecvIdx];

			if (InterpolationData.bTwoValidFrames == false)
			{
				if (InterpolationData.LastWrittenFrame != INDEX_NONE)
				{
					InterpolationData.bTwoValidFrames = true;
					TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(InterpolationData.InstanceIdx);
					FNetworkPredictionDriver<ModelDef>::SetHiddenForInterpolation(InstanceData.Info.Driver, false);
				}
				else
				{
					TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(InterpolationData.InstanceIdx);
					FNetworkPredictionDriver<ModelDef>::SetHiddenForInterpolation(InstanceData.Info.Driver, true);
				}
			}

			// Look for gaps between the current interpolation frame and what we just wrote.
			if (ToFrame != INDEX_NONE)
			{
				const int32 StartFrame = FMath::Max(FromFrame, InterpolationData.LastWrittenFrame+1);
				const int32 EndFrame = LocalFrame;

				const bool bLastWrittenFrameValid = InterpolationData.LastWrittenFrame != INDEX_NONE && InterpolationData.LastWrittenFrame > (LocalFrame - Frames.Buffer.Capacity());
				const int32 GapFromFrame = bLastWrittenFrameValid ? InterpolationData.LastWrittenFrame : LocalFrame;
				const int32 GapToFrame = LocalFrame;

				npEnsureMsgfSlow(StartFrame - EndFrame <= Frames.Buffer.Capacity(), TEXT("Gap longer than expected. StartFrame: %d. EndFrame: %d (%d)"), StartFrame, EndFrame, StartFrame - EndFrame);

				for (int32 Frame = StartFrame; Frame < EndFrame; ++Frame)
				{
					if (GapFromFrame < Frame)
					{
						// Interpolate from older data to the latest data
						Interpolate(GapFromFrame, GapToFrame, Frame, Frames, InterpolationData, FixedStepMS);
					}
					else
					{
						// Just copy latest frame
						CopyFrameData(GapToFrame, Frame, Frames, InterpolationData);
					}

					InterpolationData.LastWrittenFrame = Frame; // We just wrote something into FromFrame
					npEnsureSlow(InterpolationData.LastWrittenFrame >= 0);
				}
			}

			InterpolationData.LastWrittenFrame = LocalFrame;
			npEnsureSlow(InterpolationData.LastWrittenFrame >= 0);

			// We've taken care of this instance, reset it for next time
			DataStore->ClientRecvBitMask[ClientRecvIdx] = false;
		}
	}

	void FinalizeFrame(float DeltaTimeSeconds, const FFixedTickState* TickState) final override
	{
		const int32 FromFrame = TickState->Interpolation.ToFrame-1;
		const int32 ToFrame = TickState->Interpolation.ToFrame;
		const float PCT = TickState->Interpolation.PCT;
		const int32 InterpolatedTimeMS = TickState->Interpolation.InterpolatedTimeMS;

		npEnsureSlow(FromFrame > INDEX_NONE);
		npEnsureSlow(ToFrame > FromFrame);

		if ( NetworkPredictionCVars::DisableInterpolation() > 0)
		{
			for (TConstSetBitIterator<> BitIt(ClientRecvBitMask); BitIt; ++BitIt)
			{
				TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(BitIt.GetIndex());
				TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvData.InstanceIdx);
				TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ClientRecvData.FramesIdx);

				FNetworkPredictionDriver<ModelDef>::FinalizeFrame(InstanceData.Info.Driver, ClientRecvData.SyncState, ClientRecvData.AuxState);

				InstanceData.Info.View->UpdateView(ToFrame, ClientRecvData.SimTimeMS, &Frames.Buffer[TickState->PendingFrame].InputCmd, &ClientRecvData.SyncState, &ClientRecvData.AuxState);


				FNetworkPredictionDriver<ModelDef>::DispatchCues(&InstanceData.CueDispatcher.Get(), InstanceData.Info.Driver, ClientRecvData.ServerFrame, ClientRecvData.SimTimeMS, 0);
			}

			return;
		}



		for (auto& It : Instances)
		{
			FInstance& Instance = It;
			UE_NP_TRACE_SIM(Instance.TraceID);

			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(Instance.FramesIdx);
			
			// Ensure To/From frames are valid (replication code have been starved while local interpolation state marches on)
			if (Instance.LastWrittenFrame < ToFrame)
			{
				if (Instance.LastWrittenFrame == INDEX_NONE)
				{
					UE_NP_TRACE_SYSTEM_FAULT("No valid frames for interpolation. LastWrittenFrame: %d. ToFrame: %d", Instance.LastWrittenFrame, ToFrame);
					continue;
				}

				UE_NP_TRACE_SYSTEM_FAULT("Invalid interpolation frames. Copying old content forward. LastWrittenFrame: %d. ToFrame: %d", Instance.LastWrittenFrame, ToFrame);
				if (Instance.LastWrittenFrame < FromFrame)
				{
					CopyFrameData(Instance.LastWrittenFrame, FromFrame, Frames, Instance);
				}

				CopyFrameData(Instance.LastWrittenFrame, ToFrame, Frames, Instance);

				Instance.LastWrittenFrame = ToFrame;
				npEnsureSlow(Instance.LastWrittenFrame >= 0);
			}

			// Interpolate and dispatch
			{
				typename TInstanceFrameState<ModelDef>::FFrame& FromFrameData = Frames.Buffer[FromFrame];
				typename TInstanceFrameState<ModelDef>::FFrame& ToFrameData = Frames.Buffer[ToFrame];

				FNetworkPredictionDriver<ModelDef>::Interpolate(SyncAuxType{FromFrameData.SyncState, FromFrameData.AuxState}, SyncAuxType{ToFrameData.SyncState, ToFrameData.AuxState}, 
					PCT, Instance.SyncState, Instance.AuxState);

				// Push results to driver
				TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(Instance.InstanceIdx);
				FNetworkPredictionDriver<ModelDef>::FinalizeFrame(InstanceData.Info.Driver, Instance.SyncState, Instance.AuxState);

				// Update SimulationView with the frame we are interpolating to
				// FIXME: so this is kind of bad, we need to do '&Frames.Buffer[TickState->PendingFrame].InputCmd'  to support non-predictive (interpolated) client controlled simulations.
				// This ensures that View->PendingInputCmd (where ProduceInput service writes to) is always Frames.Buffer[TickState->PendingFrame].InputCmd (where Replicators send cmds from).
				// Really, it shouldn't be possible to disjoint the input cmds like this. This is awkward + hazardous but since it is a relatively minor, not frequently used feature, its probably ok.
				InstanceData.Info.View->UpdateView(ToFrame, InterpolatedTimeMS, &Frames.Buffer[TickState->PendingFrame].InputCmd, ToFrameData.SyncState, ToFrameData.AuxState);

				FNetworkPredictionDriver<ModelDef>::DispatchCues(&InstanceData.CueDispatcher.Get(), InstanceData.Info.Driver, FromFrame, InterpolatedTimeMS, 0);
			}
		}
	}

private:

	struct FInstance
	{
		FInstance(int32 InTraceID, int32 InInstanceIdx, int32 InFramesIdx)
			: TraceID(InTraceID), InstanceIdx(InInstanceIdx), FramesIdx(InFramesIdx) {} // fixme

		int32 TraceID;
		int32 InstanceIdx;
		int32 FramesIdx;
		int32 LastWrittenFrame = INDEX_NONE;
		bool bTwoValidFrames = false;

		// Last interpolated values. Stored here so that we can maintain FNetworkPredictionStateView to them
		TConditionalState<SyncType> SyncState;
		TConditionalState<AuxType> AuxState;
	};

	TSparseArray<FInstance> Instances; // Indices are shared with DataStore->ClientRecv
	TBitArray<> ClientRecvBitMask; // Indices into DataStore->ClientRecv that we are managing
	TBitArray<> PendingFixups; // Indices into DataStore->ClientRecv index that didn't have a valid ToFrame when reconciling

	TModelDataStore<ModelDef>* DataStore;


	static void CopyFrameData(int32 SourceFrame, int32 DestFrame, TInstanceFrameState<ModelDef>& Frames, FInstance& Instance)
	{
		typename TInstanceFrameState<ModelDef>::FFrame& SourceFrameData = Frames.Buffer[SourceFrame];
		typename TInstanceFrameState<ModelDef>::FFrame& DestFrameData = Frames.Buffer[DestFrame];

		DestFrameData.SyncState = SourceFrameData.SyncState;
		DestFrameData.AuxState = SourceFrameData.AuxState;
	}

	static void Interpolate(const int32 FromSourceFrame, const int32 ToSourceFrame, const int32 DestFrame, TInstanceFrameState<ModelDef>& Frames, FInstance& Instance, const int32 FixedStepMS)
	{
		npEnsureSlow(FromSourceFrame < ToSourceFrame);
		npEnsureSlow(FromSourceFrame < DestFrame);
		npEnsureSlow(DestFrame < ToSourceFrame);

		typename TInstanceFrameState<ModelDef>::FFrame& FromFrameData = Frames.Buffer[FromSourceFrame];
		typename TInstanceFrameState<ModelDef>::FFrame& ToFrameData = Frames.Buffer[ToSourceFrame];
		typename TInstanceFrameState<ModelDef>::FFrame& DestFrameData = Frames.Buffer[DestFrame];

		const int32 FromMS = FromSourceFrame * FixedStepMS;
		const int32 ToMS = ToSourceFrame * FixedStepMS;
		const int32 DestMS = DestFrame * FixedStepMS;

		const int32 SpreadMS = ToMS - FromMS;
		const int32 InterpolateMS = DestMS - FromMS;

		npCheckSlow(SpreadMS != 0);
		const float PCT = (float)InterpolateMS/(float)SpreadMS;

		FNetworkPredictionDriver<ModelDef>::Interpolate(SyncAuxType{FromFrameData.SyncState, FromFrameData.AuxState}, SyncAuxType{ToFrameData.SyncState, ToFrameData.AuxState}, 
			PCT, DestFrameData.SyncState, DestFrameData.AuxState);
	}
};

// ------------------------------------------------------------------------------
//	IndependentTick Interpolation
//
//	This gets messier due to non-server controlled simulations. The nature of independent
//	ticking is that they will tick at whatever rate the controlling client is running at.
//
//	This service keeps its own per-sim internal frame buffer of {Sync, Aux, Timestamp} data.
//	Most work happens in FinalizeFrame: REconcile just copies the received data to the internal buffer.
//
//	We potentially could special case server-controlled independent simulations since they will
//	at least all line up on the same frame boundaries (rep frequency will mean there are still gaps though).
//	But we don't distinguish this (server remote vs local) on simulated clients. Seems simpler to just
//	handle all SP sims the same.
// ------------------------------------------------------------------------------
class IIndependentInterpolateService
{
public:

	virtual ~IIndependentInterpolateService() = default;
	virtual void Reconcile(const FVariableTickState* TickState) = 0;
	virtual void FinalizeFrame(float DeltaTimeSeconds, const FVariableTickState* TickState) = 0;
};


template<typename InModelDef>
class TIndependentInterpolateService : public IIndependentInterpolateService
{
public:

	using ModelDef = InModelDef;
	using StateTypes = typename ModelDef::StateTypes;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;
	using SyncAuxType = TSyncAuxPair<StateTypes>;

	TIndependentInterpolateService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);
		NpResizeAndSetBit(ClientRecvBitMask, ClientRecvIdx);

		const int32 InstanceDataIdx = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx).InstanceIdx;

		FSparseArrayAllocationInfo AllocInfo = Instances.InsertUninitialized(ClientRecvIdx);
		new (AllocInfo.Pointer) FInstance(ID.GetTraceID(), InstanceDataIdx);

		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(InstanceDataIdx);
		FInstance* InternalInstance = (FInstance*)AllocInfo.Pointer;

		InstanceData.Info.View->UpdatePresentationView(InternalInstance->SyncState, InternalInstance->AuxState);
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);
		const int32 InstanceDataIdx = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx).InstanceIdx;

		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(InstanceDataIdx);
		InstanceData.Info.View->ClearPresentationView();

		ClientRecvBitMask[ClientRecvIdx] = false;
		Instances.RemoveAt(ClientRecvIdx);
	}

	void Reconcile(const FVariableTickState* TickState) final override
	{
		// DataStore->ClientRecvBitMask size can change without us knowing so make sure out InstanceBitArray size stays in sync
		NpResizeBitArray(ClientRecvBitMask, DataStore->ClientRecvBitMask.Num());

		for (TConstDualSetBitIterator<FDefaultBitArrayAllocator,FDefaultBitArrayAllocator> BitIt(ClientRecvBitMask, DataStore->ClientRecvBitMask); BitIt; ++BitIt)
		{
			const int32 ClientRecvIdx = BitIt.GetIndex();
			TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ClientRecvData.FramesIdx);
			FInstance& InterpolationData = Instances[ClientRecvIdx];

			UE_NP_TRACE_SIM(ClientRecvData.TraceID);
			const int32 LocalFrame = ++InterpolationData.LastWrittenFrame;
			
			typename FInstance::FFrame& RecvFrame = InterpolationData.ClientRecvFrames[LocalFrame];
			RecvFrame.SyncState = ClientRecvData.SyncState;
			RecvFrame.AuxState = ClientRecvData.AuxState;
			RecvFrame.SimTimeMS = ClientRecvData.SimTimeMS;

			if (LocalFrame >= 1)
			{
				npEnsureMsgfSlow(RecvFrame.SimTimeMS >= InterpolationData.ClientRecvFrames[LocalFrame-1].SimTimeMS, TEXT("Time in reverse? %d %d"), RecvFrame.SimTimeMS, InterpolationData.ClientRecvFrames[LocalFrame-1].SimTimeMS);
			}

			// We've taken care of this instance, reset it for next time
			DataStore->ClientRecvBitMask[ClientRecvIdx] = false;
		}
	}
	
	void FinalizeFrame(float DeltaTimeSeconds, const FVariableTickState* TickState) final override
	{
		const float fInterpolationTimeMS = TickState->Interpolation.fTimeMS;
		const int32 InterpolationTimeMS = (int32)TickState->Interpolation.fTimeMS;

		for (auto& It : Instances)
		{
			FInstance& Instance = It;

			if (Instance.LastWrittenFrame == INDEX_NONE)
			{
				continue;
			}

			const int32 MinFrame = FMath::Max(0, Instance.LastWrittenFrame - Instance.ClientRecvFrames.Capacity() + 1);
			const int32 MaxFrame = Instance.LastWrittenFrame;

			int32 FromFrame = MaxFrame;
			int32 ToFrame = MaxFrame;

			for (int32 Frame=MaxFrame; Frame >= MinFrame; --Frame)
			{
				if (Instance.ClientRecvFrames[Frame].SimTimeMS <= InterpolationTimeMS)
				{
					FromFrame = Frame;
					break;
				}
			}

			for (int32 Frame=FromFrame; Frame <= MaxFrame; ++Frame)
			{
				if (Instance.ClientRecvFrames[Frame].SimTimeMS >= InterpolationTimeMS)
				{
					ToFrame = Frame;
					break;
				}
			}

			npEnsureSlow(FromFrame != INDEX_NONE);
			npEnsureSlow(ToFrame != INDEX_NONE);
			npEnsureSlow(FromFrame <= ToFrame);
			npEnsureSlow(FromFrame >= MinFrame);
			npEnsureSlow(ToFrame <= MaxFrame);

			// Can happen if starved and fall behind
			//npEnsureMsgfSlow(Instance.ClientRecvFrames[FromFrame].SimTimeMS <= InterpolationTimeMS, TEXT("Unexpected FromFrame time: %d > Interpolation Time %d"), Instance.ClientRecvFrames[FromFrame].SimTimeMS, InterpolationTimeMS);
			//npEnsureMsgfSlow(Instance.ClientRecvFrames[ToFrame].SimTimeMS >= InterpolationTimeMS, TEXT("Unexpected ToFrame time: %d > Interpolation Time %d"), Instance.ClientRecvFrames[ToFrame].SimTimeMS, InterpolationTimeMS);

			TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(Instance.InstanceIdx);

			if (FromFrame != ToFrame)
			{
				typename FInstance::FFrame& FromFrameData = Instance.ClientRecvFrames[FromFrame];
				typename FInstance::FFrame& ToFromData = Instance.ClientRecvFrames[ToFrame];

				const int32 FromTimeMS = FromFrameData.SimTimeMS;
				const int32 ToTimeMS = ToFromData.SimTimeMS;
				const int32 DeltaMS = ToTimeMS - FromTimeMS;
				npEnsureSlow(DeltaMS > 0);

				const float fDeltaInterpolateMS = fInterpolationTimeMS - (float)FromTimeMS;
				npEnsureSlow(fDeltaInterpolateMS >= 0);

				const float PCT = fDeltaInterpolateMS / (float)DeltaMS;
				npEnsure(PCT >= 0.f && PCT <= 1.f);

				npEnsureMsgfSlow(Instance.ClientRecvFrames[FromFrame].SimTimeMS <= InterpolationTimeMS, TEXT("Unexpected FromFrame time: %d > Interpolation Time %d"), Instance.ClientRecvFrames[FromFrame].SimTimeMS, InterpolationTimeMS);
				npEnsureMsgfSlow(Instance.ClientRecvFrames[ToFrame].SimTimeMS >= InterpolationTimeMS, TEXT("Unexpected ToFrame time: %d > Interpolation Time %d"), Instance.ClientRecvFrames[ToFrame].SimTimeMS, InterpolationTimeMS);

				FNetworkPredictionDriver<ModelDef>::Interpolate(SyncAuxType{FromFrameData.SyncState, FromFrameData.AuxState}, SyncAuxType{ToFromData.SyncState, ToFromData.AuxState}, 
					PCT, Instance.SyncState, Instance.AuxState);
				
				FNetworkPredictionDriver<ModelDef>::FinalizeFrame(InstanceData.Info.Driver, Instance.SyncState, Instance.AuxState);

				InstanceData.Info.View->UpdateView(ToFrame, InterpolationTimeMS, nullptr, ToFromData.SyncState, ToFromData.AuxState);

				FNetworkPredictionDriver<ModelDef>::DispatchCues(&InstanceData.CueDispatcher.Get(), InstanceData.Info.Driver, FromFrame, InterpolationTimeMS, 0);
				
				if (NetworkPredictionCVars::DrawInterpolation())
				{
					FNetworkPredictionDriver<ModelDef>::DrawDebugOutline(InstanceData.Info.Driver, FColor::Green, 0.f);

					const int32 DeltaFromHead = Instance.LastWrittenFrame - FromFrame;

					FNetworkPredictionDriver<ModelDef>::DrawDebugText3D(InstanceData.Info.Driver, *FString::Printf(TEXT("[%d-%d] %d. %.2f"), FromFrame, ToFrame, DeltaFromHead, PCT), FColor::Black);
				}

				if (NetworkPredictionCVars::PrintSyncInterpolation())
				{
					UE_LOGF(LogNetworkPrediction, Display, "");
					UE_LOGF(LogNetworkPrediction, Display, "Interpolation %ls. Frames: %d - %d. InterpolationTimeMS: %d. PCT: %.2f", ModelDef::GetName(), FromFrame, ToFrame, InterpolationTimeMS, PCT);

					UE_LOGF(LogNetworkPrediction, Display, "From: %d", FromTimeMS);
					FNetworkPredictionDriver<ModelDef>::template LogUserState<SyncType>(FromFrameData.SyncState);

					UE_LOGF(LogNetworkPrediction, Display, "To: %d", ToTimeMS);
					FNetworkPredictionDriver<ModelDef>::template LogUserState<SyncType>(ToFromData.SyncState);

					UE_LOGF(LogNetworkPrediction, Display, "Interpolated: ");
					FNetworkPredictionDriver<ModelDef>::template LogUserState<SyncType>(Instance.SyncState);

					UE_LOGF(LogNetworkPrediction, Display, "");
				}
			}
			else
			{
				const int32 ActualTimeMS = Instance.ClientRecvFrames[FromFrame].SimTimeMS;

				if (FMath::Abs(ActualTimeMS - InterpolationTimeMS) > 100)
				{
					UE_LOGF(LogNetworkPrediction, Display, "[Interpolate] Interpolation out of sync. We don't have valid frames for this time for this sim. Frame %d. TimeMS: %d. @ %d", FromFrame, ActualTimeMS, InterpolationTimeMS);
				}

				typename FInstance::FFrame& FrameData = Instance.ClientRecvFrames[FromFrame];
				FNetworkPredictionDriver<ModelDef>::FinalizeFrame(InstanceData.Info.Driver, FrameData.SyncState, FrameData.AuxState);

				InstanceData.Info.View->UpdateView(FromFrame, ActualTimeMS, nullptr, FrameData.SyncState, FrameData.AuxState);

				FNetworkPredictionDriver<ModelDef>::DispatchCues(&InstanceData.CueDispatcher.Get(), InstanceData.Info.Driver, FromFrame, ActualTimeMS, 0);

				if (NetworkPredictionCVars::DrawInterpolation())
				{
					FNetworkPredictionDriver<ModelDef>::DrawDebugOutline(InstanceData.Info.Driver, FColor::Yellow, 0.f);
					FNetworkPredictionDriver<ModelDef>::DrawDebugText3D(InstanceData.Info.Driver, *FString::Printf(TEXT("[%d] %d/%d (%d)"), FromFrame, ActualTimeMS, InterpolationTimeMS, (InterpolationTimeMS-ActualTimeMS)), FColor::Black);
				}
			}
		}
	}

private:
	
	struct FInstance
	{
		FInstance(int32 InTraceID, int32 InInstanceIdx)
			: TraceID(InTraceID), InstanceIdx(InInstanceIdx), ClientRecvFrames(16) { } // fixme

		int32 TraceID;
		int32 InstanceIdx;

		int32 LastWrittenFrame = INDEX_NONE;
		int32 LastFromFrame = INDEX_NONE;

		// Last interpolated values. Stored here so that we can maintain FNetworkPredictionStateView to them
		TConditionalState<SyncType> SyncState;
		TConditionalState<AuxType> AuxState;

		// Store received frames locally on the service.
		// This is currently the only service that benefits from doing this,
		// so implementing it on the service itself. 
		//
		// It would be nice if we could configure the NetRecv to use an array of TClientRecvData.
		// This would also let us handle 'multiple NetRecvs in a frame' better here.
		// All other services just care about 'latest received' state interpolating would benefit
		// from storing all received frames. But this adds complications for just this one case.
		// TClientRecvData also stores more than we would need to here - the InputCmd and ServerFrame aren't
		// relevant for independent interpolation. This just seems simpler.

		struct FFrame
		{
			TConditionalState<SyncType> SyncState;
			TConditionalState<AuxType> AuxState;
			int32 SimTimeMS;
		};

		TNetworkPredictionBuffer<FFrame> ClientRecvFrames;
	};

	TBitArray<> ClientRecvBitMask; // Indices into DataStore->ClientRecv that we are managing
	TSparseArray<FInstance> Instances; // Indices are shared with DataStore->ClientRecv

	TModelDataStore<ModelDef>* DataStore;
};
