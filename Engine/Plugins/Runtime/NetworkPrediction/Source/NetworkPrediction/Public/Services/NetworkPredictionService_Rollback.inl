// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "NetworkPredictionCVars.h"
#include "NetworkPredictionLog.h"
#include "Services/NetworkPredictionInstanceData.h"
#include "Services/NetworkPredictionService_Ticking.inl"

namespace NetworkPredictionCVars
{
	NETSIM_DEVCVAR_SHIPCONST_INT(ForceReconcile,			0, "np.ForceReconcile",				"Force a single reconcile back to the last server-acknoledged frame. When used with np.ForceReconcileExtraFrames, additional frames can be rolled back. No effect on server. Resets after use.");
	NETSIM_DEVCVAR_SHIPCONST_INT(ForceReconcileExtraFrames, 0, "np.ForceReconcileExtraFrames",	"Roll back this extra number of frames during the next targeted reconcile. Must be positive and reasonable given the buffer sizes.");
	NETSIM_DEVCVAR_SHIPCONST_INT(SkipReconcile,				0, "np.SkipReconcile",				"Skip all reconciles");
	NETSIM_DEVCVAR_SHIPCONST_INT(PrintReconciles,			0, "np.PrintReconciles",			"Print reconciles to log");
}

class IFixedRollbackService
{
public:

	virtual ~IFixedRollbackService() = default;
	virtual int32 QueryRollback(const FFixedTickState* TickState) = 0;

	virtual void PreStepRollback(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep, const int32 Offset, const bool bFirstStepInResim) = 0;
	virtual void StepRollback(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep) = 0;
};

template<typename InModelDef>
class TFixedRollbackService : public IFixedRollbackService
{
public:

	using ModelDef = InModelDef;
	using StateTypes = typename ModelDef::StateTypes;
	using SyncAuxType = TSyncAuxPair<StateTypes>;

	static constexpr bool bNeedsTickService = FNetworkPredictionDriver<ModelDef>::HasSimulation();

	TFixedRollbackService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore), InternalTickService(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);
		NpResizeAndSetBit(InstanceBitArray, ClientRecvIdx);

		if (bNeedsTickService)
		{
			InternalTickService.RegisterInstance(ID);
		}
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);
		InstanceBitArray[ClientRecvIdx] = false;
		
		if (bNeedsTickService)
		{
			InternalTickService.UnregisterInstance(ID);
		}
	}

	int32 QueryRollback(const FFixedTickState* TickState) final override
	{
		npCheckSlow(TickState);
		NpClearBitArray(RollbackBitArray);

		// DataStore->ClientRecvBitMask size can change without us knowing so make sure out InstanceBitArray size stays in sync
		NpResizeBitArray(InstanceBitArray, DataStore->ClientRecvBitMask.Num());

		const int32 Offset = TickState->Offset;
		int32 RollbackFrame = INDEX_NONE;
		for (TConstDualSetBitIterator<FDefaultBitArrayAllocator,FDefaultBitArrayAllocator> BitIt(InstanceBitArray, DataStore->ClientRecvBitMask); BitIt; ++BitIt)
		{
			const int32 ClientRecvIdx = BitIt.GetIndex();
			TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ClientRecvData.FramesIdx);

			UE_NP_TRACE_SIM(ClientRecvData.TraceID);
			
			const int32 LocalFrame = ClientRecvData.ServerFrame - Offset;
			typename TInstanceFrameState<ModelDef>::FFrame& LocalFrameData = Frames.Buffer[LocalFrame];

			TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvData.InstanceIdx);

			bool bDoRollback = false;

			if (NetworkPredictionCVars::ForceReconcile() > 0)
			{
				UE_NP_TRACE_SHOULD_RECONCILE(ClientRecvData.TraceID);
				bDoRollback = true;
				RollbackFrame = LocalFrame - FMath::Max(0, NetworkPredictionCVars::ForceReconcileExtraFrames());

				if (NetworkPredictionCVars::PrintReconciles())
				{				
					UE_LOGF(LogNetworkPrediction, Warning, "Reconcile activated due to ForceReconcile (to RollbackFrame=%i, including %i extra rollback frames)", RollbackFrame, -(RollbackFrame-LocalFrame));
				}

				NetworkPredictionCVars::SetForceReconcile(0); // reset
			}
			else if (FNetworkPredictionDriver<ModelDef>::ShouldReconcile(InstanceData.Info.Driver, SyncAuxType(LocalFrameData.SyncState, LocalFrameData.AuxState), SyncAuxType(ClientRecvData.SyncState, ClientRecvData.AuxState)))
			{
				UE_NP_TRACE_SHOULD_RECONCILE(ClientRecvData.TraceID);
				bDoRollback = true;
				
				if (NetworkPredictionCVars::PrintReconciles())
				{
					UE_LOGF(LogNetworkPrediction, Warning, "Reconcile required due to Sync/Aux mismatch. LocalFrame: %d. Recv Frame: %d. Offset: %d. Idx: %d", LocalFrame, ClientRecvData.ServerFrame, Offset, LocalFrame % Frames.Buffer.Capacity());

					UE_LOGF(LogNetworkPrediction, Warning, "Received:");
					FNetworkPredictionDriver<ModelDef>::LogUserStates({ClientRecvData.InputCmd, ClientRecvData.SyncState, ClientRecvData.AuxState });

					UE_LOGF(LogNetworkPrediction, Warning, "Local:");
					FNetworkPredictionDriver<ModelDef>::LogUserStates({LocalFrameData.InputCmd, LocalFrameData.SyncState, LocalFrameData.AuxState });
				}
			}

			if (bDoRollback && !NetworkPredictionCVars::SkipReconcile())
			{
				RollbackFrame = (RollbackFrame == INDEX_NONE) ? LocalFrame : FMath::Min(RollbackFrame, LocalFrame);
			}
			else
			{
				// Copy received InputCmd to head. This feels a bit out of place here but is ok for now.
				//	-If we rollback, this isn't needed since rollback will copy the cmd (someone else could cause the rollback though, making this redundant)
				//	-Making a second "no rollback happening" pass on all SPs is an option but the branch here seems better, this is the only place we are touching the head frame buffer though...
				if (ClientRecvData.NetRole == ROLE_SimulatedProxy)
				{
					typename TInstanceFrameState<ModelDef>::FFrame& PendingFrameData = Frames.Buffer[TickState->PendingFrame];
					PendingFrameData.InputCmd = ClientRecvData.InputCmd;
				}
			}

			// Regardless if this instance needs to rollback or not, we are marking it in the RollbackBitArray.
			// This could be a ModelDef setting ("Rollback everyone" or "Just who needs it") 
			// Or maybe something more dynamic/spatial ("rollback all instances within this radius", though to do this you may need to consider some ModelDef independent way of doing so)
			NpResizeAndSetBit(RollbackBitArray, ClientRecvIdx);

			// We've taken care of this instance, reset it for next time
			DataStore->ClientRecvBitMask[ClientRecvIdx] = false;
		}
		
		return RollbackFrame;
	}

	void PreStepRollback(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep, const int32 Offset, const bool bFirstStepInResim)
	{
		if (bFirstStepInResim)
		{
			// Apply corrections for the instances that have corrections on this frame
			ApplyCorrection<false>(ServiceStep.LocalInputFrame, Offset);

			// Everyone must rollback Cue dispatcher and flush
			InternalTickService.BeginRollback(ServiceStep.LocalInputFrame, Step.TotalSimulationTime, Step.Frame);
			
			// Everyone we are managing needs to rollback to this frame, even if they don't have a correction 
			// (this frame or this rollback - they will need to restore their collision data since we are about to retick everyone in step)

			QUICK_SCOPE_CYCLE_COUNTER(NP_Rollback_RestoreFrame);
			TRACE_CPUPROFILER_EVENT_SCOPE(NetworkPrediction::RestoreFrame);

			for (TConstSetBitIterator<> BitIt(InstanceBitArray); BitIt; ++BitIt)
			{
				TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(BitIt.GetIndex());
				TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvData.InstanceIdx);
				TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ClientRecvData.FramesIdx);
				typename TInstanceFrameState<ModelDef>::FFrame& LocalFrameData = Frames.Buffer[ServiceStep.LocalInputFrame];

				FNetworkPredictionDriver<ModelDef>::RestoreFrame(InstanceData.Info.Driver, LocalFrameData.SyncState.Get(), LocalFrameData.AuxState.Get());
			}
		}
		else
		{
			ApplyCorrection<true>(ServiceStep.LocalInputFrame, Offset);
		}
	}

	void StepRollback(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep) final override
	{
		if (bNeedsTickService)
		{
			InternalTickService.TickResim(Step, ServiceStep);
		}
	}	

private:

	template<bool FlushCorrection>
	void ApplyCorrection(const int32 LocalInputFrame, const int32 Offset)
	{
		// Insert correction data on the right frame
		for (TConstSetBitIterator<> BitIt(RollbackBitArray); BitIt; ++BitIt)
		{
			const int32 ClientRecvIdx = BitIt.GetIndex();
			TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);

			const int32 LocalFrame = ClientRecvData.ServerFrame - Offset;
			if (LocalFrame == LocalInputFrame)
			{
				// Time to inject
				TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ClientRecvData.FramesIdx);
				typename TInstanceFrameState<ModelDef>::FFrame& LocalFrameData = Frames.Buffer[LocalFrame];

				LocalFrameData.SyncState = ClientRecvData.SyncState;
				LocalFrameData.AuxState = ClientRecvData.AuxState;

				TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ClientRecvData.InstanceIdx);

				// Copy input cmd if SP
				if (ClientRecvData.NetRole == ROLE_SimulatedProxy)
				{
					LocalFrameData.InputCmd = ClientRecvData.InputCmd;
				}

				RollbackBitArray[ClientRecvIdx] = false;
				UE_NP_TRACE_ROLLBACK_INJECT(ClientRecvData.TraceID);

				if (FlushCorrection)
				{
					// Push to component/collision scene immediately (we aren't garunteed to tick next, so get our collision right)
					FNetworkPredictionDriver<ModelDef>::RestoreFrame(InstanceData.Info.Driver, LocalFrameData.SyncState.Get(), LocalFrameData.AuxState.Get());
				}
			}
		}
	}
	
	TBitArray<> InstanceBitArray; // Indices into DataStore->ClientRecv that we are managing
	TBitArray<> RollbackBitArray; // Indices into DataStore->ClientRecv that we should rollback

	TModelDataStore<ModelDef>* DataStore;

	TLocalTickService<ModelDef>	InternalTickService;
};

// ------------------------------------------------------------------------------------------------

class IIndependentRollbackService
{
public:

	virtual ~IIndependentRollbackService() = default;
	virtual void Reconcile(const FVariableTickState* TickState) = 0;
};

template<typename InModelDef>
class TIndependentRollbackService : public IIndependentRollbackService
{
public:

	using ModelDef = InModelDef;
	using StateTypes = typename ModelDef::StateTypes;
	using SyncAuxType = TSyncAuxPair<StateTypes>;

	TIndependentRollbackService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);
		NpResizeAndSetBit(InstanceBitArray, ClientRecvIdx);

		// Only APs should register for this service. We do not support rollback for independent tick SP actors.
		npEnsureSlow(DataStore->Instances.GetByIndexChecked( DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx).InstanceIdx ).NetRole == ROLE_AutonomousProxy);
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 ClientRecvIdx = DataStore->ClientRecv.GetIndexChecked(ID);
		InstanceBitArray[ClientRecvIdx] = false;
	}

	void Reconcile(const FVariableTickState* TickState) final override
	{
		// DataStore->ClientRecvBitMask size can change without us knowing so make sure out InstanceBitArray size stays in sync
		NpResizeBitArray(InstanceBitArray, DataStore->ClientRecvBitMask.Num());

		for (TConstDualSetBitIterator<FDefaultBitArrayAllocator,FDefaultBitArrayAllocator> BitIt(InstanceBitArray, DataStore->ClientRecvBitMask); BitIt; ++BitIt)
		{
			const int32 ClientRecvIdx = BitIt.GetIndex();
			TClientRecvData<ModelDef>& ClientRecvData = DataStore->ClientRecv.GetByIndexChecked(ClientRecvIdx);
			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ClientRecvData.FramesIdx);

			const int32 LocalFrame = ClientRecvData.ServerFrame;
			typename TInstanceFrameState<ModelDef>::FFrame& LocalFrameData = Frames.Buffer[LocalFrame];

			TInstanceData<ModelDef>& Instance = DataStore->Instances.GetByIndexChecked(ClientRecvData.InstanceIdx);

			if (FNetworkPredictionDriver<ModelDef>::ShouldReconcile( Instance.Info.Driver, SyncAuxType(LocalFrameData.SyncState, LocalFrameData.AuxState), SyncAuxType(ClientRecvData.SyncState, ClientRecvData.AuxState) ))
			{
				UE_NP_TRACE_SHOULD_RECONCILE(ClientRecvData.TraceID);
				if (NetworkPredictionCVars::PrintReconciles())
				{
					UE_LOGF(LogNetworkPrediction, Warning, "ShouldReconcile. Frame: %d.", LocalFrame);

					UE_LOGF(LogNetworkPrediction, Warning, "Received:");
					FNetworkPredictionDriver<ModelDef>::LogUserStates({ClientRecvData.InputCmd, ClientRecvData.SyncState, ClientRecvData.AuxState });

					UE_LOGF(LogNetworkPrediction, Warning, "Local:");
					FNetworkPredictionDriver<ModelDef>::LogUserStates({LocalFrameData.InputCmd, LocalFrameData.SyncState, LocalFrameData.AuxState });
				}

				LocalFrameData.SyncState = ClientRecvData.SyncState;
				LocalFrameData.AuxState = ClientRecvData.AuxState;

				FNetworkPredictionDriver<ModelDef>::RestoreFrame(Instance.Info.Driver, LocalFrameData.SyncState.Get(), LocalFrameData.AuxState.Get());

				// Do rollback
				const int32 EndFrame = TickState->PendingFrame;
				for (int32 Frame = LocalFrame; Frame < EndFrame; ++Frame)
				{
					const int32 InputFrame = Frame;
					const int32 OutputFrame = Frame+1;

					typename TInstanceFrameState<ModelDef>::FFrame& InputFrameData = Frames.Buffer[InputFrame];
					typename TInstanceFrameState<ModelDef>::FFrame& OutputFrameData = Frames.Buffer[OutputFrame];

					const FVariableTickState::FFrame& TickData = TickState->Frames[InputFrame];

					FNetSimTimeStep Step {TickData.DeltaMS, TickData.TotalMS, OutputFrame };

					const int32 EndTimeMS = TickData.TotalMS + TickData.DeltaMS;

					TTickUtil<ModelDef>::DoTick(Instance, InputFrameData, OutputFrameData, Step, EndTimeMS, ESimulationTickContext::Resimulate);

					UE_NP_TRACE_PUSH_TICK(Step.TotalSimulationTime, Step.StepMS, Step.Frame);
					UE_NP_TRACE_SIM_TICK(ClientRecvData.TraceID);
				}
			}

			// We've taken care of this instance, reset it for next time
			DataStore->ClientRecvBitMask[ClientRecvIdx] = false;
		}
	}

private:

	TBitArray<> InstanceBitArray; // Indices into DataStore->ClientRecv that we are managing
	TModelDataStore<ModelDef>* DataStore;
};
