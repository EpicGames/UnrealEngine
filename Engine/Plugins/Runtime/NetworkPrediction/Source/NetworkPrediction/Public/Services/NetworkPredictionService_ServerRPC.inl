// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionDriver.h"
#include "Services/NetworkPredictionInstanceData.h"

// The ServerRPCService's job is to tell the Driver to call the Server RPC that sends InputCmds upstream.
// This happens every frame on autonomous proxy clients.
//
// The implementation here is very basic. It may make sense to have some scalability at this level,
// options for throttling send rate, etc.

class IServerRPCService
{
public:

	virtual ~IServerRPCService() = default;
	virtual void CallServerRPC(float DeltaTimeSeconds) = 0;
};

template<typename InModelDef>
class TServerRPCService : public IServerRPCService	
{
public:

	using ModelDef = InModelDef;
	using DriverType = typename ModelDef::Driver;

	TServerRPCService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		TInstanceData<ModelDef>* InstanceData = DataStore->Instances.Find(ID);
		npCheckSlow(InstanceData);

		Instances.Add((int32)ID, FInstance{InstanceData->TraceID, InstanceData->Info.Driver});
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		Instances.Remove((int32)ID);
	}

	void CallServerRPC(float DeltaTimeSeconds) final override
	{
		for (auto& MapIt : Instances)
		{
			FInstance& Instance = MapIt.Value;
			FNetworkPredictionDriver<ModelDef>::CallServerRPC(Instance.Driver);
		}
	}

private:

	// The vast majority of the time there will be <= 1 instances that wants to call the ServerRPC.
	// Only split screen type situations will require more.
	struct FInstance
	{
		int32 TraceID;
		DriverType* Driver;
	};

	TSortedMap<int32, FInstance, TInlineAllocator<1>> Instances;
	TModelDataStore<ModelDef>* DataStore;
};
