// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NetworkPredictionCVars.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionTrace.h"
#include "Services/NetworkPredictionInstanceData.h"


namespace NetworkPredictionCVars
{
	NETSIM_DEVCVAR_SHIPCONST_INT(DisableSmoothing, 0, "np.Smoothing.Disable", "Disables smoothing and just finalizes using the latest simulation state");
}


// ------------------------------------------------------------------------------
//	FixedTick Smoothing
//
//  This first pass service simply performs interpolation between the most recent fixed tick states
//  and passes the smoothed state to the driver to handle however it chooses. 
//	
//  Future improvements could include smoothing out corrections after a reconcile, and expanding
//  that to smoothing for Independent ticking mode.
// ------------------------------------------------------------------------------
class IFixedSmoothingService
{
public:

	virtual ~IFixedSmoothingService() = default;
	virtual void UpdateSmoothing(const FServiceTimeStep& ServiceStep,const FFixedTickState* TickState) = 0;
	virtual void FinalizeSmoothingFrame(const FFixedTickState* TickState) = 0;
};

template<typename InModelDef>
class TFixedSmoothingService : public IFixedSmoothingService
{
public:

	using ModelDef = InModelDef;
	using StateTypes = typename ModelDef::StateTypes;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;
	using SyncAuxType = TSyncAuxPair<StateTypes>;
	

	TFixedSmoothingService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 InstanceDataIdx = DataStore->Instances.GetIndex(ID);

		const int32 PreviousSize = Instances.GetAllocatedSize();

		FSparseArrayAllocationInfo AllocInfo = Instances.InsertUninitialized(InstanceDataIdx);

		// When the array resizes, other instances may have dangling pointers to Presentation/PrevPresentation views and will need updated
		const bool bNeedsAllViewsUpdated = PreviousSize != Instances.GetAllocatedSize();

		new (AllocInfo.Pointer) FInstance(ID.GetTraceID(), InstanceDataIdx, DataStore->Frames.GetIndex(ID));

		if (bNeedsAllViewsUpdated)
		{
			UpdateAllInstancePresentationViews();
		}
		else
		{
			TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(InstanceDataIdx);

			// Point the PresentationView to our managed state. Note this only has to be done once
			FInstance* InternalInstance = (FInstance*)AllocInfo.Pointer;
			InstanceData.Info.View->UpdatePresentationView(InternalInstance->SyncState, InternalInstance->AuxState);
			InstanceData.Info.View->UpdatePrevPresentationView(InternalInstance->LastSyncState, InternalInstance->LastAuxState);
		}
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 Idx = DataStore->Instances.GetIndex(ID);
		TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(Idx);
		InstanceData.Info.View->ClearPresentationView();
		Instances.RemoveAt(Idx);
	}

	virtual void UpdateSmoothing(const FServiceTimeStep& ServiceStep, const FFixedTickState* TickState) final override
	{
		const int32 OutputFrame = ServiceStep.LocalOutputFrame;
		
		for (auto& It : Instances)
		{
			FInstance& Instance = It;
			
			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(Instance.FramesIdx);

			typename TInstanceFrameState<ModelDef>::FFrame& OutputFrameData = Frames.Buffer[OutputFrame];
			if (NetworkPredictionCVars::DisableSmoothing() != 0 || !Instance.bHasTwoFrames)
			{
				OutputFrameData.SyncState.CopyTo(Instance.SyncState);
				OutputFrameData.AuxState.CopyTo(Instance.AuxState);
				OutputFrameData.SyncState.CopyTo(Instance.LastSyncState);
				OutputFrameData.AuxState.CopyTo(Instance.LastAuxState);
				Instance.bHasTwoFrames = true;
				continue;
			}

			// TODO: could improve this with a double-buffer that alternates, eliminating one copy
			// Set Last Presentation States To Current
			Instance.SyncState.CopyTo(Instance.LastSyncState);
			Instance.AuxState.CopyTo(Instance.LastAuxState);

			// Set Sync State To The Pending State
			OutputFrameData.SyncState.CopyTo(Instance.SyncState);
			OutputFrameData.AuxState.CopyTo(Instance.AuxState);
		}
	}
	
	virtual void FinalizeSmoothingFrame(const FFixedTickState* TickState) final override
	{
		const float RemainingTime = TickState->UnspentTimeMS;
		const float TimeStep = TickState->FixedStepMS;
		const float Alpha = FMath::Clamp(RemainingTime / TimeStep,0.f,1.f);
		
		if (NetworkPredictionCVars::DisableSmoothing() != 0)
		{
			for (auto& It : Instances)
			{
				const FInstance& Instance = It;
				{
					// Push non-smoothed results to driver
					const TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(Instance.InstanceIdx);
					FNetworkPredictionDriver<ModelDef>::FinalizeSmoothingFrame(InstanceData.Info.Driver, Instance.SyncState.Get(), Instance.AuxState.Get());
				}
			}

			return;
		}

		
		for (auto& It : Instances)
		{
			const FInstance& Instance = It;
			
			// Interpolate and push smoothed state to driver
			{
				TConditionalState<SyncType> SmoothedSyncState;
				TConditionalState<AuxType> SmoothedAuxState;

				FNetworkPredictionDriver<ModelDef>::Interpolate(SyncAuxType{Instance.LastSyncState.Get(), Instance.LastAuxState.Get()}, SyncAuxType{Instance.SyncState.Get(), Instance.AuxState.Get()},
					Alpha, SmoothedSyncState, SmoothedAuxState);

				// Push smoothed results to driver
				const TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(Instance.InstanceIdx);
				FNetworkPredictionDriver<ModelDef>::FinalizeSmoothingFrame(InstanceData.Info.Driver, SmoothedSyncState, SmoothedAuxState);
				
			}
		}
	}

private:
	void UpdateAllInstancePresentationViews()
	{
		for (auto& It : Instances)
		{
			FInstance& InternalInstance = It;

			TInstanceData<ModelDef>& RemapInstanceData = DataStore->Instances.GetByIndexChecked(InternalInstance.InstanceIdx);
			RemapInstanceData.Info.View->UpdatePresentationView(InternalInstance.SyncState, InternalInstance.AuxState);
			RemapInstanceData.Info.View->UpdatePrevPresentationView(InternalInstance.LastSyncState, InternalInstance.LastAuxState);
		}
	}

	struct FInstance
	{
		FInstance(int32 InTraceID, int32 InInstanceIdx, int32 InFramesIdx)
			: TraceID(InTraceID), InstanceIdx(InInstanceIdx), FramesIdx(InFramesIdx) {}

		int32 TraceID;
		int32 InstanceIdx;
		int32 FramesIdx;
		uint8 bHasTwoFrames : 1 = false;
		
		// Latest states to smooth between. Stored here so that we can maintain FNetworkPredictionStateView to them
		TConditionalState<SyncType> SyncState;
		TConditionalState<AuxType> AuxState;
		TConditionalState<SyncType> LastSyncState;
		TConditionalState<AuxType> LastAuxState;
		
	};
	
	TSparseArray<FInstance> Instances; // Indices are shared with DataStore->ClientRecv
	
	TModelDataStore<ModelDef>* DataStore;
};

