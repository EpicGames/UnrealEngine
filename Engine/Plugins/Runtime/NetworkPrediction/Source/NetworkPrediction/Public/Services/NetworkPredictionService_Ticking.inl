// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "NetworkPredictionInstanceData.h"
#include "NetworkPredictionTickState.h"
#include "NetworkPredictionTrace.h"
#include "NetworkPredictionUtil.h"

// Common util used by the ticking services. Might make sense to move to FNetworkPredictionDriverBase if needed elsewhere
template<typename ModelDef>
struct TTickUtil
{
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	using FrameDataType = typename TInstanceFrameState<ModelDef>::FFrame;

	template<typename SimulationType = typename ModelDef::Simulation>
	static typename TEnableIf<!std::is_same_v<SimulationType, void>>::Type DoTick(TInstanceData<ModelDef>& Instance, FrameDataType& InputFrameData, FrameDataType& OutputFrameData, const FNetSimTimeStep& Step, const int32 CueTimeMS, ESimulationTickContext TickContext)
	{
		Instance.CueDispatcher->PushContext({Step.Frame, CueTimeMS, TickContext});

		// Update cached view before calling tick. If something tries to do an OOB mod to this simulation, it 
		// can only write to the output/pending state. (Input state is frozen now).
		FNetworkPredictionStateView* View = Instance.Info.View;
		View->bTickInProgress = true;
		View->UpdateView(Step.Frame, Step.TotalSimulationTime, &OutputFrameData.InputCmd, &OutputFrameData.SyncState, &OutputFrameData.AuxState);

		// FIXME: aux. Copy it over and make a fake lazy writer for now
		OutputFrameData.AuxState = InputFrameData.AuxState;
		TNetSimLazyWriterFunc<AuxType> LazyAux((void*)&OutputFrameData.AuxState);
		
		Instance.Info.Simulation->SimulationTick( Step,
			{ InputFrameData.InputCmd, InputFrameData.SyncState, InputFrameData.AuxState }, // TNetSimInput
			{ OutputFrameData.SyncState, LazyAux, Instance.CueDispatcher.Get() } ); // TNetSimOutput

		View->bTickInProgress = false;
		Instance.CueDispatcher->PopContext();

		// Fixme: should only trace aux if it changed
		UE_NP_TRACE_USER_STATE_SYNC(ModelDef, OutputFrameData.SyncState.Get());
		UE_NP_TRACE_USER_STATE_AUX(ModelDef, OutputFrameData.AuxState.Get());
	}

	template<typename SimulationType = typename ModelDef::Simulation>
	static typename TEnableIf<std::is_same_v<SimulationType, void>>::Type DoTick(TInstanceData<ModelDef>& Instance, FrameDataType& InputFrameData, FrameDataType& OutputFrameData, const FNetSimTimeStep& Step, const int32 EndTimeMS, ESimulationTickContext TickContext)
	{
		npCheckf(false, TEXT("DoTick called on %s with no Simulation defined"), ModelDef::GetName());
	}
};

// The tick service's role is to tick new simulation frames based on local frame state (fixed or independent/variable)
class ILocalTickService
{
public:

	virtual ~ILocalTickService() = default;
	virtual void Tick(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep) = 0;
};

template<typename InModelDef>
class TLocalTickServiceBase : public ILocalTickService
{
public:

	using ModelDef = InModelDef;
	using StateTypes = typename ModelDef::StateTypes;
	using InputType = typename StateTypes::InputType;
	using SyncType = typename StateTypes::SyncType;
	using AuxType = typename StateTypes::AuxType;

	TLocalTickServiceBase(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		InstancesToTick.Add((int32)ID, FInstance {ID.GetTraceID(), DataStore->Instances.GetIndex(ID), DataStore->Frames.GetIndex(ID)} );
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		InstancesToTick.Remove((int32)ID);
	}

	void Tick(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep) override
	{
		Tick_Internal<false>(Step, ServiceStep);
	}

	void TickResim(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep)
	{
		Tick_Internal<true>(Step, ServiceStep);
	}

	void BeginRollback(const int32 LocalFrame, const int32 StartTimeMS, const int32 ServerFrame)
	{
		for (auto It : InstancesToTick)
		{
			TInstanceData<ModelDef>& Instance = DataStore->Instances.GetByIndexChecked(It.Value.InstanceIdx);
			UE_NP_TRACE_SIM(Instance.TraceID);
			Instance.CueDispatcher->NotifyRollback(ServerFrame);
		}
	}

protected:

	template<bool bIsResim>
	void Tick_Internal(const FNetSimTimeStep& Step, const FServiceTimeStep& ServiceStep)
	{
		const int32 InputFrame = ServiceStep.LocalInputFrame;
		const int32 OutputFrame = ServiceStep.LocalOutputFrame;

		const int32 StartTime = Step.TotalSimulationTime;
		const int32 EndTime = ServiceStep.EndTotalSimulationTime;

		for (auto It : InstancesToTick)
		{
			TInstanceData<ModelDef>& Instance = DataStore->Instances.GetByIndexChecked(It.Value.InstanceIdx);
			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(It.Value.FrameBufferIdx);

			typename TInstanceFrameState<ModelDef>::FFrame& InputFrameData = Frames.Buffer[InputFrame];
			typename TInstanceFrameState<ModelDef>::FFrame& OutputFrameData = Frames.Buffer[OutputFrame];

			UE_NP_TRACE_SIM_TICK(It.Value.TraceID);

			// Copy current input into the output frame. This is redundant in the case where we are polling
			// local input but is needed in the other cases. Simpler to just copy it always.
			if (!bIsResim || Instance.NetRole == ROLE_SimulatedProxy)
			{
				OutputFrameData.InputCmd = InputFrameData.InputCmd;
			}

			TTickUtil<ModelDef>::DoTick(Instance, InputFrameData, OutputFrameData, Step, EndTime, GetTickContext<bIsResim>(Instance.NetRole));
		}
	}

	template<bool bIsResim>
	ESimulationTickContext GetTickContext(ENetRole NetRole)
	{
		if (bIsResim)
		{
			switch(NetRole)
			{
			case ENetRole::ROLE_AutonomousProxy:
				return ESimulationTickContext::Resimulate;
				break;
			case ENetRole::ROLE_SimulatedProxy:
				return ESimulationTickContext::Resimulate;
				break;
			}
		}
		else
		{
			switch(NetRole)
			{
			case ENetRole::ROLE_Authority:
				return ESimulationTickContext::Authority;
				break;
			case ENetRole::ROLE_AutonomousProxy:
				return ESimulationTickContext::Predict;
				break;
			case ENetRole::ROLE_SimulatedProxy:
				return ESimulationTickContext::Predict; // Fixme: all sim proxies are forward predicted now. We need to look at net LOD here?
				break;
			}
		}

		npEnsureMsgf(false, TEXT("Unexpected NetRole %d during regular tick"), NetRole);
		return ESimulationTickContext::None;
	}

	struct FInstance
	{
		int32 TraceID;
		int32 InstanceIdx; // idx into TModelDataStore::Instances
		int32 FrameBufferIdx; // idx into TModelDataStore::Frames
	};

	TSortedMap<int32, FInstance> InstancesToTick;
	TModelDataStore<ModelDef>* DataStore;
	
};

// To allow template specialization
template<typename InModelDef>
class TLocalTickService : public TLocalTickServiceBase<InModelDef>
{
public:
	TLocalTickService(TModelDataStore<InModelDef>* InDataStore) 
		: TLocalTickServiceBase<InModelDef>(InDataStore)
	{

	}
};

// -------------------------------------------------------------


// Service for ticking independent simulations that are remotely controlled.
// E.g, only used by the server for ticking remote clients that are in independent ticking mode,
class IRemoteIndependentTickService
{
public:

	virtual ~IRemoteIndependentTickService() = default;
	virtual void Tick(float DeltaTimeSeconds, const FVariableTickState* VariableTickState) = 0;
};

// Ticking remote clients on the server. 
template<typename InModelDef>
class TRemoteIndependentTickService : public IRemoteIndependentTickService
{
public:
	using ModelDef = InModelDef;

	// These are rough ballparks, maybe should be configurable
	static constexpr int32 MinRemoteClientStepMS = 1;
	static constexpr int32 MaxRemoteClientStepMS = 100;

	static constexpr int32 MaxRemoteClientStepsPerFrame = 6;
	static constexpr int32 MaxRemoteClientTotalMSPerFrame = 200;

	TRemoteIndependentTickService(TModelDataStore<ModelDef>* InDataStore)
		: DataStore(InDataStore) { }

	void RegisterInstance(FNetworkPredictionID ID)
	{
		const int32 ServerRecvIdx = DataStore->ServerRecv_IndependentTick.GetIndexChecked(ID);
		NpResizeAndSetBit(InstanceBitArray, ServerRecvIdx);
	}

	void UnregisterInstance(FNetworkPredictionID ID)
	{
		const int32 ServerRecvIdx = DataStore->ServerRecv_IndependentTick.GetIndexChecked(ID);
		InstanceBitArray[ServerRecvIdx] = false;
	}

	void Tick(float DeltaTimeSeconds, const FVariableTickState* VariableTickState) final override
	{
		npEnsureSlow(VariableTickState->PendingFrame >= 0);

		const float fEngineFrameDeltaTimeMS = DeltaTimeSeconds * 1000.f;
		const int32 CueTimeMS = VariableTickState->Frames[VariableTickState->PendingFrame].TotalMS; // This time stamp is what will get replicated to SP clients for Cues.

		for (TConstSetBitIterator<> BitIt(InstanceBitArray); BitIt; ++BitIt)
		{
			const int32 ServerRecvIdx = BitIt.GetIndex();
			TServerRecvData_Independent<ModelDef>& ServerRecvData = DataStore->ServerRecv_IndependentTick.GetByIndexChecked(ServerRecvIdx);
			ServerRecvData.UnspentTimeMS += fEngineFrameDeltaTimeMS;

			TInstanceFrameState<ModelDef>& Frames = DataStore->Frames.GetByIndexChecked(ServerRecvData.FramesIdx);
			TInstanceData<ModelDef>& InstanceData = DataStore->Instances.GetByIndexChecked(ServerRecvData.InstanceIdx);

			int32 TotalFrames = 0;
			int32 TotalMS = 0;

			const int32 TraceID = ServerRecvData.TraceID;

			while (ServerRecvData.LastConsumedFrame < ServerRecvData.LastRecvFrame)
			{
				const int32 NextFrame = ++ServerRecvData.LastConsumedFrame;
				typename TServerRecvData_Independent<ModelDef>::FFrame& NextRecvData = ServerRecvData.InputBuffer[NextFrame];

				if (NextRecvData.DeltaTimeMS == 0)
				{
					// Dropped cmd, just skip and pretend nothing happened (expect client to be corrected)
					continue;
				}

				const int32 InputCmdMS = FMath::Clamp(NextRecvData.DeltaTimeMS, MinRemoteClientStepMS, MaxRemoteClientTotalMSPerFrame);

				if (InputCmdMS > (int32)ServerRecvData.UnspentTimeMS)
				{
					break;
				}

				const int32 NewTotalMS = TotalMS + InputCmdMS;
				if (NewTotalMS > MaxRemoteClientTotalMSPerFrame)
				{
					break;
				}

				// Tick
				{
					TotalMS = NewTotalMS;
					ServerRecvData.UnspentTimeMS -= (float)InputCmdMS;
					if (FMath::IsNearlyZero(ServerRecvData.UnspentTimeMS))
					{
						ServerRecvData.UnspentTimeMS = 0.f;
					}

					const int32 InputFrame = ServerRecvData.PendingFrame++;
					const int32 OutputFrame = ServerRecvData.PendingFrame;

					Frames.Buffer[InputFrame].InputCmd = NextRecvData.InputCmd;

					typename TInstanceFrameState<ModelDef>::FFrame& InputFrameData = Frames.Buffer[InputFrame];
					typename TInstanceFrameState<ModelDef>::FFrame& OutputFrameData = Frames.Buffer[OutputFrame];

					FNetSimTimeStep Step {InputCmdMS, ServerRecvData.TotalSimTimeMS, OutputFrame };

					ServerRecvData.TotalSimTimeMS += InputCmdMS;

					UE_NP_TRACE_PUSH_TICK(Step.TotalSimulationTime, Step.StepMS, Step.Frame);
					UE_NP_TRACE_SIM_TICK(TraceID);

					TTickUtil<ModelDef>::DoTick(InstanceData, InputFrameData, OutputFrameData, Step, CueTimeMS, ESimulationTickContext::Authority);
					
				}

				if (++TotalFrames == MaxRemoteClientStepsPerFrame)
				{
					break;
				}
			}
		}
	}

private:

	TBitArray<> InstanceBitArray; // Indices into DataStore->ServerRecv_IndependentTick that we are managing
	TModelDataStore<ModelDef>* DataStore;
};
